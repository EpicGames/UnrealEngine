{
  "FileVersion" : 3,
  "Version" : 1,
  "VersionName" : "0.1",
  "FriendlyName" : "Proxy LOD Plugin (Experimental)",
  "Description" : "A plugin to generate Proxy LOD systems.",
  "Category" : "Editor",
  "CreatedBy" : "Epic Games, Inc.",
  "CreatedByURL" : "https://epicgames.com",
  "DocsURL" : "",
  "MarketplaceURL" : "",
  "SupportURL" : "",
  "EnabledByDefault" : false,
  "CanContainContent" : false,
  "IsBetaVersion" : true,
  "Installed" : false,
  "Modules" :
	[
		{
			"Name" : "ProxyLODMeshReduction",
			"Type" : "Editor",
			"LoadingPhase" : "Default",
			 "PlatformAllowList": [ "Win64" ]
		}
	]
}