// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <array>

#include "Math/Vector.h"   // For FVector
#include "ProxyLODTwoDTriangleUtilities.h"

namespace ProxyLOD
{

	/** 
	* Double precision array of length 3, used primarily for Barycentric coords.
	*/
	typedef std::array<double, 3> DArray3d;

	// Interpolate Vertex Data to the location given in Barycentric Coords.
	template <typename T>
	static T InterpolateVertexData(const DArray3d& BarycentricPos, const T(&VertexData)[3])
	{
		// In order BarycentricCoords = normalized distance from {V1V2 edge,  V2V0 edge, V0V1 edge}.
		return BarycentricPos[0] * VertexData[0] + BarycentricPos[1] * VertexData[1] + BarycentricPos[2] * VertexData[2];
	}


	/**
	* Compute Scaled Distances for all three sided of a 2d triangle V1V2, V2V0, V0V1 to the given point in the plane.
	* 
	* @param  Vertices  Define a triangle on a 2d plane.
	* @param  Point     A 2d point, form which to measure the scaled distances.
	*
	* @param  OutScaledDistances   Scaled distances to the three edges (V1-V2, V2-V0, V0-V1). 
	*/
	static inline void ComputeScaledDistances(const DTriangle2d& Vertices, const DVec2d& Point, DArray3d& OutScaledDistances)
	{
		OutScaledDistances[0] = ScaledDistanceToLine(Vertices[1], Vertices[2], Point); // alpha  = 0 on V1V2
		OutScaledDistances[1] = ScaledDistanceToLine(Vertices[2], Vertices[0], Point); // beta   = 0 on V2V0
		OutScaledDistances[2] = ScaledDistanceToLine(Vertices[0], Vertices[1], Point); // gamma  = 0 on V0V1

	}

	/**
	* Convert pre-computed Scaled Distances to Barycentric coordinates for a 2d triangle V1V2, V2V0, V0V1.
	*
	* NB: In practice, use ComputeScaledDistances to arrive at signed scaled distances between triangle edges and a point
	*     If the signs all indicate (i.e. all positive) the point is inside the given triangle, 
	*     use this function to convert to Barycentric coords.
	*
	* @param  Vertices                 Define a triangle on a 2d plane.
	* @param  InOutScaledDistances     Precomputed scaled distances to a 2d point to be converted to Barycentric on return.
	*
	*/
	static inline void ConvertScaledDistanceToBarycentric(const DTriangle2d& Vertices, DArray3d& InOutScaledDistances)
	{
		InOutScaledDistances[0] /= ScaledDistanceToLine(Vertices[1], Vertices[2], Vertices[0]); // alpha = 1 at V[0]
		InOutScaledDistances[1] /= ScaledDistanceToLine(Vertices[2], Vertices[0], Vertices[1]); // beta  = 1 at V[1]
		InOutScaledDistances[2] /= ScaledDistanceToLine(Vertices[0], Vertices[1], Vertices[2]); // gamma = 1 at V[2]
	}

	/**
	* Compute the barycentric weights at 2D @param Point assuming the 2D vertices are given in counter clockwise order.
	*
	* NB: This is for 2D triangles and points.  
	*
	* @param  Vertices                 Define a triangle on a 2d plane.
	* @param  Point                    Point on the plane. 
	*
	* @return Barycentric coordinates (alpha, beta, gamma) of the point relative to the input triangle.
	*         alpha = 1 at Vertices[0],  beta = 1 at Vertices[1] and gamma = 1 at Vertices[2] 
	*/
	static DArray3d ComputeBarycentricWeights(const DTriangle2d& Vertices, const DVec2d& Point)
	{

		// The result
		DArray3d Weights;

		ComputeScaledDistances(Vertices, Point, Weights);


		// The scaled distance between the point and each vertex.   Note Weight[i] = 1 when Point = Vi

		ConvertScaledDistanceToBarycentric(Vertices, Weights);


		return Weights;
	}

	// Computes the Barycentric weights for a point projected onto the plane of a triangle.
	/**
	* Compute the barycentric weights at 3D @param Point projected onto the plane of triangle described in 3d.
	* assuming triangle ertices are given in counter clockwise order.
	*
	* NB: This is for 3D triangles and points.
	*
	* @param  Vertices                 Define a triangle.
	* @param  Point                    Point in 3d space.
	*
	* @return Barycentric coordinates (alpha, beta, gamma) of the point relative to the input triangle.
	*         alpha = 1 at Vertices[0],  beta = 1 at Vertices[1] and gamma = 1 at Vertices[2]
	*/
	static DArray3d ComputeBarycentricWeights(const FVector3f(&Vertices)[3], const FVector3f& Point)
	{

		if (1)
		{
			const FVector3f U = Vertices[1] - Vertices[0];
			const FVector3f V = Vertices[2] - Vertices[0];
			const FVector3f W = Point - Vertices[0];
			const FVector3f NormDir = FVector3f::CrossProduct(U, V);
			const float NSqr = FVector3f::DotProduct(NormDir, NormDir);

			FVector3f Tmp = FVector3f::CrossProduct(U, W);
			double Gamma = FVector3f::DotProduct(Tmp, NormDir) / NSqr;

			Tmp = FVector3f::CrossProduct(W, V);
			double Beta = FVector3f::DotProduct(Tmp, NormDir) / NSqr;

			DArray3d Result = { 1. - Gamma - Beta, Beta, Gamma };
			return Result;
		}
		else
		{
			// Alternate calculation of the weights at the projected point.
			// Should be equivalent (more explicit but slower)

			const FVector3f E10 = Vertices[1] - Vertices[0];
			const FVector3f E20 = Vertices[2] - Vertices[0];
			const FVector3f Ndir = FVector3f::CrossProduct(E10, E20);
			float LengthSqr = FVector3f::DotProduct(Ndir, Ndir);


			// project point
			const FVector3f Pprime = Point - Ndir * FVector3f::DotProduct(Point, Ndir) / LengthSqr;

			const FVector3f E21 = Vertices[2] - Vertices[1];
			const FVector3f Na =  FVector3f::CrossProduct(E21, Pprime - Vertices[1]);
			const FVector3f Nb = -FVector3f::CrossProduct(E20, Pprime - Vertices[2]);
			const FVector3f Nc =  FVector3f::CrossProduct(E10, Pprime - Vertices[0]);

			float Alpha = FVector3f::DotProduct(Na, Ndir) / LengthSqr;
			float Beta  = FVector3f::DotProduct(Nb, Ndir) / LengthSqr;
			float Gamma = FVector3f::DotProduct(Nc, Ndir) / LengthSqr;
			DArray3d Result = { Alpha, Beta, Gamma };
			return Result;
		}

	}


}
