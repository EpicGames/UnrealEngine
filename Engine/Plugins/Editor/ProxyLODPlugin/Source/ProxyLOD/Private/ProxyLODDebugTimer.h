// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Misc/ScopedSlowTask.h"
#include "Stats/StatsMisc.h"



#if 0
	#define PROXYLOD_DEBUG_TIMER(name_) SCOPE_LOG_TIME(name_, nullptr)
#else
	#define PROXYLOD_DEBUG_TIMER(name_) (void)name_;
#endif