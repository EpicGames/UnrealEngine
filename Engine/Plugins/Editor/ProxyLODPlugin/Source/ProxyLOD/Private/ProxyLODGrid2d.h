// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Math/IntPoint.h"
#include "Misc/AssertionMacros.h"

namespace ProxyLOD
{
	/**
	* Definition of the texture atlas texel resolution and gutter size, also in texels.
	*/
	struct FTextureAtlasDesc
	{
		FTextureAtlasDesc()
			: Size()
			, Gutter()
		{
		}

		FTextureAtlasDesc(const FIntPoint& S, const double G)
			: Size(S)
			, Gutter(G)
		{
		}

		FIntPoint Size;
		float Gutter;
	};


	/**
	* Template two-dimensional grid class that manages its own memory.
	* 
	* NB: The first direction is the fast direction in memory.
	*/
	template< typename GridDataType>
	class TGrid
	{
	public:
		typedef TSharedPtr<TGrid>   Ptr;
		typedef GridDataType        ValueType;

		/**
		* Construct a grid managed by a shared pointer
		* @param  INum  number of elements in the 1st direction.
		* @param  JNum  number of elements in the 2nd direction.
		*/
		static Ptr Create(int32 INum, int32 JNum)
		{
			Ptr TwoDGrid(new TGrid(INum, JNum));
			return TwoDGrid;
		}

		/**
		* Constructor
		* @param  INum  number of elements in the 1st direction.
		* @param  JNum  number of elements in the 2nd direction.
		*/
		TGrid(uint32 INum, uint32 JNum); 
		

		/**
		* Destructor. Deletes any allocated memory.
		*/
		~TGrid();
		

		/**
		*  The Shape of this two dimensional grid.
		*
		* @return (Imax, Jmax) the length of the 1st and 2nd dimensions mananged by this grid.
		*/
		FIntPoint Size() const;

		/**
		*  Const and non-const access memory in 2d layout fashion. 
		*
		* NB: i should be in the range [0, IMax -1]
		*     j should be in the range [0, JMax -1]
		*
		* @param i  The coordinate in first direction.
		* @param j  The coordinate in second direction.
		*
		* @return const/nonconst reference to Value nominally stored at (i,j)
		*/
		const ValueType& Get(const uint32 i, const uint32 j) const;
		ValueType& Get(const uint32 i, const uint32 j);
		

		/**
		*  Const and non-const access memory in 2d layout fashion.
		*
		* NB: i should be in the range [0, IMax -1]
		*     j should be in the range [0, JMax -1]
		*
		* @param i  The coordinate in first direction.
		* @param j  The coordinate in second direction.
		*
		* @return const/nonconst reference to Value nominally stored at (i,j)
		*/
		const ValueType& operator()(const uint32 i, const uint32 j) const;
		ValueType& operator()(const uint32 i, const uint32 j);

		/**
		*  Const and non-const access memory in linear fashion
		*
		* @return const/nonconst pointer to the head of the memory
		*/
		const ValueType*  GetData() const
		{
			return DataArray;
		}
		ValueType* GetData()
		{
			return DataArray;
		}



		/**
		* Swap the data payload of this grid with another grid containing the same data type payload.
		*/
		void Swap(TGrid<ValueType>& other);
	

	private:

		// Can't copy me
		TGrid(const TGrid& other) {}

		uint32 IMax;
		uint32 JMax;
		GridDataType* DataArray;
	};

	

	/**
	* Template wrapped used to give two-dimensional grid semantics to a pre-existing linear tarray.
	*
	* NB: The first direction is the fast direction in memory.
	*     This wrapper does not manage the memory of the TArray.
	*/
	template <typename ValueType>
	class TGridWrapper
	{
	public:
		/**
		* Constructor used to map a TArray to a grid.
		* 
		* NB: It is required that the shape and the size fo the grid are 
		*     consitent: Shape.X * Shape.Y == LinearArray.Num()
		*
		* @param  LinearArray  The underlying array to be expressed at a 2d grid.
		* @param  Shape defines the (Imax, Jmax) extent of the grid
		* 
		*/
		TGridWrapper(TArray<ValueType>& LinearArray, const FIntPoint& Shape);

		/**
		*  The Shape of this two dimensional grid.
		*
		* @return (Imax, Jmax) the length of the 1st and 2nd dimensions managed by this grid.
		*/
		FIntPoint Size() const;

		/**
		*  Const and non-const access memory in 2d layout fashion.
		*
		* NB: i should be in the range [0, IMax -1]
		*     j should be in the range [0, JMax -1]
		*
		* @param i  The coordinate in first direction.
		* @param j  The coordinate in second direction.
		*
		* @return const/nonconst reference to Value nominally stored at (i,j)
		*/
		const ValueType& Get(const int32 i, const int32 j) const;
		ValueType& Get(const int32 i, const int32 j);
		
		/**
		*  Const and non-const access memory in 2d layout fashion.
		*
		* NB: i should be in the range [0, IMax -1]
		*     j should be in the range [0, JMax -1]
		*
		* @param i  The coordinate in first direction.
		* @param j  The coordinate in second direction.
		*
		* @return const/nonconst reference to Value nominally stored at (i,j)
		*/
		const ValueType& operator()(const int32 i, const int32 j) const;
		ValueType& operator()(const int32 i, const int32 j);
	
	
		/**
		*  Const and non-const access memory in linear fashion
		*
		* @return const/nonconst pointer to the head of the memory
		*/
		const ValueType*  GetData() const
		{
			return DataArray;
		}
		ValueType* GetData()
		{
			return DataArray;
		}


	private:
		// No.  Can only construct with a TArray
		TGridWrapper();
		// No need to copy this wrapper.
		TGridWrapper(const TGridWrapper&);

		int32  IMax;
		int32  JMax;
		ValueType* DataArray;
	};


	// Actual methods of templated classes.

	template <typename ValueType>
	TGridWrapper<ValueType>::TGridWrapper(TArray<ValueType>& LinearArray, const FIntPoint& Shape) :
		IMax(Shape.X),
		JMax(Shape.Y),
		DataArray(LinearArray.GetData())
	{
		check(Shape.X * Shape.Y == LinearArray.Num());
	}

	template <typename ValueType>
	FIntPoint TGridWrapper<ValueType>::Size() const
	{
		return FIntPoint(IMax, JMax);
	}

	template <typename ValueType>
	const ValueType&  TGridWrapper<ValueType>::Get(const int32 i, const int32 j) const
	{
		checkSlow(i < IMax && j < JMax);
		checkSlow(i > -1 && j > -1);
		// make i the fast variable
		const size_t Idx = i + j * IMax;
		return DataArray[Idx];
	}

	template <typename ValueType>
	ValueType&  TGridWrapper<ValueType>::Get(const int32 i, const int32 j)
	{
		checkSlow(i < IMax && j < JMax);
		checkSlow(i > -1 && j > -1);

		// make i the fast variable
		const size_t Idx = i + j * IMax;
		return DataArray[Idx];
	}

	template <typename ValueType>
	const ValueType& TGridWrapper<ValueType>::operator()(const int32 i, const int32 j) const
	{
		return Get(i, j);
	}

	template <typename ValueType>
	ValueType& TGridWrapper<ValueType>::operator()(const int32 i, const int32 j)
	{
		return Get(i, j);
	}




	template< typename GridDataType>
	TGrid<GridDataType>::TGrid(uint32 INum, uint32 JNum) :
		IMax(INum), JMax(JNum)
	{
		// Allocate the memory
		DataArray = new GridDataType[IMax * JMax];

	}

	template< typename GridDataType>
	TGrid<GridDataType>::~TGrid()
	{
		if (DataArray) delete[] DataArray;
	}

	template< typename GridDataType>
	FIntPoint TGrid<GridDataType>::Size() const
	{
		return FIntPoint(IMax, JMax);
	}
	template< typename GridDataType>
	const GridDataType&  TGrid<GridDataType>::Get(const uint32 i, const uint32 j) const
	{
		checkSlow(i < IMax && j < JMax);
		// make i the fast variable
		const size_t Idx = i + j * IMax;
		return DataArray[Idx];
	}
	template< typename GridDataType>
	GridDataType& TGrid<GridDataType>::Get(const uint32 i, const uint32 j)
	{
		checkSlow(i < IMax && j < JMax);
		// make i the fast variable
		const size_t Idx = i + j * IMax;
		return DataArray[Idx];
	}
	template< typename GridDataType>
	const GridDataType& TGrid<GridDataType>::operator()(const uint32 i, const uint32 j) const
	{
		return Get(i, j);
	}
	template< typename GridDataType>
	GridDataType& TGrid<GridDataType>::operator()(const uint32 i, const uint32 j)
	{
		return Get(i, j);
	}
	template< typename GridDataType>
	void TGrid<GridDataType>::Swap(TGrid<GridDataType>& other)
	{
		uint32 IMaxTemp        = IMax;
		uint32 JMaxTemp        = JMax;
		GridDataType* DataTemp = DataArray;

		IMax      = other.IMax;
		JMax      = other.JMax;
		DataArray = other.DataArray;

		other.IMax      = IMaxTemp;
		other.JMax      = JMaxTemp;
		other.DataArray = DataTemp;
	}

}
