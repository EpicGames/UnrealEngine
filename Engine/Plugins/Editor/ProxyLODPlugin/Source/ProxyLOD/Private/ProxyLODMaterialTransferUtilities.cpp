// Copyright Epic Games, Inc. All Rights Reserved.


#include "ProxyLODMaterialTransferUtilities.h"

#include "ProxyLODkDOPInterface.h"
#include "ProxyLODThreadedWrappers.h"

#include "ProxyLODMeshConvertUtils.h"
#include "Math/Matrix.h" // used in chirality..

#include "StaticMeshAttributes.h"


namespace ProxyLOD
{
	typedef TGrid<FLinearColor>  FLinearColorGrid;
}

namespace
{
	// returns the HitUV
	template<typename TestFunctorType>
	float EvaluateHit(const FVector3f& HitPoint, const uint32 SrcPolyId, const FMeshDescriptionArrayAdapter& MeshAdapter,
		ProxyLOD::FSrcMeshData& HitPayload, TestFunctorType& TestFunctor)
	{
		int32 MeshId, InstanceIdx, LocalFaceNumber;
		const FMeshDescriptionArrayAdapter::FRawPoly SrcPoly = MeshAdapter.GetRawPoly(SrcPolyId, MeshId, InstanceIdx, LocalFaceNumber, ERawPolyValues::WedgeTexCoords | ERawPolyValues::VertexPositions | ERawPolyValues::WedgeTangents);
		int32 MaterialIdx = SrcPoly.FaceMaterialIndex;

		FVector2D UVs[3];
		for (int i = 0; i < 3; ++i)
		{
			UVs[i] = SrcPoly.WedgeTexCoords[0][i];
		}

		// Get Additional data about this poly - might have alternate UVs that we need to use
		const FMeshMergeData& MeshMergeData = MeshAdapter.GetMeshMergeData(MeshId);

		const bool bHasNewUVs = (MeshMergeData.NewUVs.Num() != 0);

		if (bHasNewUVs)
		{
			const TArray<FVector2D>& NewUVs = MeshMergeData.NewUVs;

			for (int32 i = 0, Offset = LocalFaceNumber * 3; i < 3; ++i)
			{
				UVs[i] = NewUVs[Offset + i];
			}
		}

		// might need to rescale old uvs instead.
		if (!bHasNewUVs && MeshMergeData.TexCoordBounds.IsValidIndex(MaterialIdx))
		{
			const FBox2D& Bounds = MeshMergeData.TexCoordBounds[MaterialIdx];
			if (Bounds.GetArea() > 0)
			{
				const FVector2D MinUV = Bounds.Min;
				const FVector2D ScaleUV(1.0f / (Bounds.Max.X - Bounds.Min.X), 1.0f / (Bounds.Max.Y - Bounds.Min.Y));

				for (int32 i = 0; i < 3; ++i)
				{
					// NB: FVector2D * FVector2D is componetwise.
					UVs[i] = (UVs[i] - MinUV) * ScaleUV;
				}
			}
		}

		HitPayload.MaterialId = MaterialIdx;
		HitPayload.TriangleId = SrcPolyId;

		// Record the triangle-local coordinates of this hit point.
		HitPayload.BarycentricCoords = ProxyLOD::ComputeBarycentricWeights(SrcPoly.VertexPositions, HitPoint);

		// Record the UV coordinates of the hit point.
		HitPayload.UV = ProxyLOD::InterpolateVertexData(HitPayload.BarycentricCoords, UVs);

		// Is the orientation of the tangent space reversed?
		{
			float Chirality[3];
			for (int32 i = 0; i < 3; ++i)
			{
				// StaticMeshVertexBuffer.h computes it this way...(see SetTangents)
				FMatrix Basis(
					FPlane((FVector)SrcPoly.WedgeTangentX[i], 0),
					FPlane((FVector)SrcPoly.WedgeTangentY[i], 0),
					FPlane((FVector)SrcPoly.WedgeTangentZ[i], 0),
					FPlane(0, 0, 0, 1)
				);
				Chirality[i] = (Basis.Determinant() < 0) ? -1.f : 1.f;
			}
			float Handedness = FMath::Clamp(ProxyLOD::InterpolateVertexData(HitPayload.BarycentricCoords, Chirality), -1.f, 1.f);
			HitPayload.Handedness = (Handedness > 0) ? 1 : -1;
		}

		// Is the SrcNormal roughly in the same direction as the ray
		// the face normal of this poly
		return TestFunctor(SrcPoly);
	}

	float EvaluateHit(const FVector3f& HitPoint, const uint32 SrcPolyId, const FMeshDescriptionArrayAdapter& MeshAdapter,
		ProxyLOD::FSrcMeshData& HitPayload)
	{
		auto NoOpFunctor = [](const FMeshDescriptionArrayAdapter::FRawPoly& RawPoly)->float {return 1; };

		return EvaluateHit(HitPoint, SrcPolyId, MeshAdapter, HitPayload, NoOpFunctor);
	};

	// returns the HitUV
	template <bool Forward>
	float EvaluateHit(const FkHitResult& HitResult, const FVector3f& RayStart, const FVector3f& RayDirection, const FMeshDescriptionArrayAdapter& MeshAdapter,
		ProxyLOD::FSrcMeshData& HitPayload)
	{

		// Get the Poly we hit

		const uint32 SrcPolyId = HitResult.Item;

		// Interpolate the UVs to the hit point

		const float  HitTime = (Forward) ? HitResult.Time : -HitResult.Time;
		const FVector3f HitPoint = RayStart + HitTime * RayDirection;

		auto DirectionalTest = [&RayDirection](const FMeshDescriptionArrayAdapter::FRawPoly& RawPoly)->float
		{
			// Is the SrcNormal roughly in the same direction as the ray
			// the face normal of this poly
			// const FVector3f SrcNormal = ComputeNormal(RawPoly.VertexPositions); 

			FVector3f SrcNormal = RawPoly.WedgeTangentZ[0] + RawPoly.WedgeTangentZ[1] + RawPoly.WedgeTangentZ[2];
			SrcNormal.Normalize();

			return FVector3f::DotProduct(SrcNormal, RayDirection);
		};

		HitPayload.Forward = (Forward == true) ? 1 : 0;

		return EvaluateHit(HitPoint, SrcPolyId, MeshAdapter, HitPayload, DirectionalTest);
	}

}

// Generate a map between UV locations on the simplified geometry and points on the Src geometry
// Inactive texels in the result have MaterialId = -1;

ProxyLOD::FSrcDataGrid::Ptr ProxyLOD::CreateCorrespondence(
	const FMeshDescriptionArrayAdapter& SrcMesh,
	const FkDOPTree& SrckDOPTree,
	const FVertexDataMesh& ReducedMesh,
	const ProxyLOD::FRasterGrid& UVGrid,
	const int32 TransferType,
	float MaxRayLength)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::CreateCorrespondence)

	const int32 TestTransferType = TransferType;

	// The dimensions of the resulting buffer match the UV Grid.

	const FIntPoint Size = UVGrid.Size();

	// Create the target

	ProxyLOD::FSrcDataGrid::Ptr TargetGridPtr = ProxyLOD::FSrcDataGrid::Create(Size.X, Size.Y);
	ProxyLOD::FSrcDataGrid& TargetGrid = *TargetGridPtr;

	FUnitTransformDataProvider kDOPDataProvider(SrckDOPTree);

	// Access to the simplified mesh data.

	const auto& Indices = ReducedMesh.Indices;
	const auto& Points = ReducedMesh.Points;
	const auto& Normal = (ReducedMesh.TransferNormal.Num() == 0) ? ReducedMesh.Normal : ReducedMesh.TransferNormal;

	checkSlow(Normal.Num() == Points.Num());

	// Iterate over the the UVGrid.

	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, Size.Y),
		[&](const ProxyLOD::FIntRange& Range)
	{

		for (int j = Range.begin(); j < Range.end(); ++j)
		{
			for (int i = 0; i < Size.X; ++i)
			{
				const auto& TexelData = UVGrid(i, j);
				ProxyLOD::FSrcMeshData& TargetTexel = TargetGrid(i, j);
			
				// initialize with values that will be over-written if we
				// can create a correspondence with the source mesh for this texel.

				TargetTexel.MaterialId = -1;

				// This texel is associated with a triangle on the target geometry.
				if (TexelData.TriangleId > -1)
				{
					const uint32 TriangleId = TexelData.TriangleId;

					// The Barycentric Coords we identify with this texel.
					// NB: when Data.SignedDistance > 0 this is a point on the closest
					// triangle.

					// In order BarycentricCoords = distance from {V0-V1 edge,  V1-V2 edge, V2-V1 edge}.

					const auto& BarycentricCoords = TexelData.BarycentricCoords;

					// The vertices of the triangle in question.

					const uint32 Offset = TriangleId * 3;

					// The triangle in the simplified mesh that corresponds to this texel

					const FVector3f VertPos[3] = { Points[Indices[Offset + 0]], Points[Indices[Offset + 1]], Points[Indices[Offset + 2]] };
					const FVector3f VertNormal[3] = { Normal[Indices[Offset + 0]], Normal[Indices[Offset + 1]], Normal[Indices[Offset + 2]] };

					// Find the location in 3d space the corresponds to this UV on this triangle,
					// this will be the origin for rays we shoot.
					// NB: This is interpolating based on the texture-space barycentric-coords.  
					// This could introduce some distortion. 

					const FVector3f RayOrigin = ProxyLOD::InterpolateVertexData(BarycentricCoords, VertPos);

					// face normal.. used for testing
					//  FVector3f RayDirection = ComputeNormal(VertPos);

					// The averaged normal at this position, revert to face normal if this fails.
					FVector3f RayDirection = ProxyLOD::InterpolateVertexData(BarycentricCoords, VertNormal);
					bool bSuccess = RayDirection.Normalize(1.e-5f); // Todo: what if this normalize fails?
					if (!bSuccess)
					{
						RayDirection = ComputeNormal(VertPos);
					}


					// Slight offset to the star point when casting rays to insure that we don't miss the case
					// when the Src geometry is actually co-planar with triangle we are shooting from.

					const FVector3f ForwardRayStart = RayOrigin - 0.0001 * RayDirection;
					const FVector3f ReverseRayStart = RayOrigin + 0.0001 * RayDirection;

					// We have to shoot rays in forward & back-facing normal direction to look for the closest source triangle. 
					// Construct a forward and a reverse ray.
					FVector3f ForwardRayEnd = ForwardRayStart + RayDirection * MaxRayLength;
					FVector3f ReverseRayEnd = ReverseRayStart - RayDirection * MaxRayLength;

					// Find t

					FkHitResult ForwardResult;

					TkDOPLineCollisionCheck<const FUnitTransformDataProvider, uint32>  ForwardRay((FVector)ForwardRayStart, (FVector)ForwardRayEnd, true, kDOPDataProvider, &ForwardResult);

					FkHitResult ReverseResult;

					TkDOPLineCollisionCheck<const FUnitTransformDataProvider, uint32>  ReverseRay((FVector)ReverseRayStart, (FVector)ReverseRayEnd, true, kDOPDataProvider, &ReverseResult);


					// Fire both rays

					bool bForwardHit = SrckDOPTree.LineCheck(ForwardRay);
					bool bReverseHit = SrckDOPTree.LineCheck(ReverseRay);


					//int32 HitTriIdx = -1;
					//int32 HitForward = -2;

					ProxyLOD::FSrcMeshData HitPayload;
					HitPayload.MaterialId = -1;
					HitPayload.UV = FVector2D(0.f, 0.f);
					if (TestTransferType == 0)
					{

						if (bForwardHit && bReverseHit) // Both directions hit.
						{


							ProxyLOD::FSrcMeshData ForwardHitPayload;
							const float ForwardHitNormalDotNormal = EvaluateHit<true>(ForwardResult, ForwardRayStart, RayDirection, SrcMesh, ForwardHitPayload);
							const bool bForwardCodirectional = (ForwardHitNormalDotNormal > 0);

							ProxyLOD::FSrcMeshData ReverseHitPayload;
							const float ReverseHitNormalDotNormal = EvaluateHit<false>(ReverseResult, ReverseRayStart, RayDirection, SrcMesh, ReverseHitPayload);
							const bool bReverseCodirectional = (ReverseHitNormalDotNormal > 0);

							// If either are co-aligned, we will use it.
							HitPayload.CoAligned = (bForwardCodirectional || bReverseCodirectional) ? 1 : -1;

							if (bForwardCodirectional && bReverseCodirectional) // Both hits were "valid"
							{

								// pick the closest
								if (ForwardResult.Time < ReverseResult.Time)
								{
									HitPayload = ForwardHitPayload;
									HitPayload.Forward = 1;
								}
								else
								{
									HitPayload = ReverseHitPayload;
									HitPayload.Forward = 0;
								}
							}
							else if (bForwardCodirectional) // only the forward hit was "valid"
							{
								HitPayload = ForwardHitPayload;
								HitPayload.Forward = 1;
							}
							else if (bReverseCodirectional) // only the reverse hit was "valid"
							{
								HitPayload = ReverseHitPayload;
								HitPayload.Forward = 0;
							}
							else
							{
								HitPayload = ReverseHitPayload;
								HitPayload.Forward = -1;
							}
						}
						else if (bForwardHit) // just forward hit
						{
							const float NormalDotNormal = EvaluateHit<true>(ForwardResult, ForwardRayStart, RayDirection, SrcMesh, HitPayload);
							HitPayload.CoAligned = (NormalDotNormal > 0) ? 1 : -1;
						}
						else if (bReverseHit) // just reverse hit
						{
							const float NormalDotNormal = EvaluateHit<false>(ReverseResult, ReverseRayStart, RayDirection, SrcMesh, HitPayload);
							HitPayload.CoAligned = (NormalDotNormal > 0) ? 1 : -1;
						}
					}// end test type 0
					else
					{
						float NormalDotNormal;

						// both directions hit.
						if (bForwardHit && bReverseHit)
						{
							// just pick the closest
							if (ForwardResult.Time < ReverseResult.Time)
							{
								NormalDotNormal = EvaluateHit<true>(ForwardResult, ForwardRayStart, RayDirection, SrcMesh, HitPayload);
							}
							else
							{
								NormalDotNormal = EvaluateHit<false>(ReverseResult, ReverseRayStart, RayDirection, SrcMesh, HitPayload);
							}
						}
						else if (bForwardHit)
						{
							NormalDotNormal = EvaluateHit<true>(ForwardResult, ForwardRayStart, RayDirection, SrcMesh, HitPayload);

						}
						else if (bReverseHit)
						{
							NormalDotNormal = EvaluateHit<false>(ReverseResult, ReverseRayStart, RayDirection, SrcMesh, HitPayload);
						}
						else
						{
							NormalDotNormal = 1;
						}
						HitPayload.CoAligned = (NormalDotNormal > 0.f) ? 1 : -1;

					}
					// 
					if (HitPayload.MaterialId != -1)
					{
						TargetTexel = HitPayload;
					}

				}

			}
		}
	});

	return TargetGridPtr;
}


// Generate a map between UV locations on the simplified geometry and points on the Src geometry
// Inactive texels in the result have MaterialId = -1;
ProxyLOD::FSrcDataGrid::Ptr ProxyLOD::CreateCorrespondence(
	const openvdb::Int32Grid& ClosestPolyGrid,
	const FMeshDescriptionArrayAdapter& SrcMesh,
	const FkDOPTree& SrckDOPTree,
	const FVertexDataMesh& ReducedMesh,
	const ProxyLOD::FRasterGrid& UVGrid,
	const int32 TransferType, 
	float MaxRayLength)
{

	const int32 TestTransferType = TransferType;

	// The dimensions of the resulting buffer match the UV Grid.

	const FIntPoint Size = UVGrid.Size();

	// Create the target

	ProxyLOD::FSrcDataGrid::Ptr TargetGridPtr = ProxyLOD::FSrcDataGrid::Create(Size.X, Size.Y);
	ProxyLOD::FSrcDataGrid& TargetGrid = *TargetGridPtr;

	// Use already constructed k-DOP tree that holds the src geometry.
	// This is used later when we have to form a correspondence between
	// simplified geometry and src geometry by shooting rays.
	FUnitTransformDataProvider kDOPDataProvider(SrckDOPTree);

	// Access to the simplified mesh data.

	const auto& Indices = ReducedMesh.Indices;
	const auto& Points = ReducedMesh.Points;
	const auto& Normal = (ReducedMesh.TransferNormal.Num() == 0) ? ReducedMesh.Normal : ReducedMesh.TransferNormal;

	checkSlow(Normal.Num() == Points.Num());



	// Iterate over the the UVGrid.

	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, Size.Y),
		[&](const ProxyLOD::FIntRange& Range)
	{

		// Accessor used to find the closest poly used in generating the isosurface

		openvdb::Int32Grid::ConstAccessor ClosestPolyIdAccessor = ClosestPolyGrid.getConstAccessor();
		const auto& XForm = ClosestPolyGrid.transform();


		for (int j = Range.begin(); j < Range.end(); ++j)
		{
			for (int i = 0; i < Size.X; ++i)
			{
				const auto& TexelData = UVGrid(i, j);
				ProxyLOD::FSrcMeshData& TargetTexel = TargetGrid(i, j);
				TargetTexel.MaterialId = -1;

				if (TexelData.TriangleId > -1)
				{
					const uint32 TriangleId = TexelData.TriangleId;

					// The Barycentric Coords we identify with this texel.
					// NB: when Data.SignedDistance > 0 this is a point on the closest
					// triangle.

					// In order BarycentricCoords = distance from {V0-V1 edge,  V1-V2 edge, V2-V1 edge}.

					const auto& BarycentricCoords = TexelData.BarycentricCoords;

					// The vertices of the triangle in question.

					const uint32 Offset = TriangleId * 3;

					// The triangle in the simplified mesh that corresponds to this texel

					const FVector3f VertPos[3] = { Points[Indices[Offset + 0]], Points[Indices[Offset + 1]], Points[Indices[Offset + 2]] };

					// Find the location in 3d space the corresponds to this UV on this triangle,
					// this will be the origin for rays we shoot.
					// NB: This is interpolating based on the texture-space barycentric-coords.  
					// This could introduce some distortion. 

					const FVector3f RayOrigin = ProxyLOD::InterpolateVertexData(BarycentricCoords, VertPos);


					ProxyLOD::FSrcMeshData HitPayload;
					HitPayload.MaterialId = -1;
					HitPayload.UV = FVector2D(0.f, 0.f);

					// Find the closest poly using the vdb index grid.  This should only return src polys that contributed
					// to the resulting iso-surface.

					openvdb::Int32 ClosestPolyId;
					const openvdb::Coord ijk = XForm.worldToIndexCellCentered(openvdb::Vec3d(RayOrigin.X, RayOrigin.Y, RayOrigin.Z));
					bool bFoundClosePoly = ClosestPolyIdAccessor.probeValue(ijk, ClosestPolyId);

					if (bFoundClosePoly)
					{

						// compute the barycentric coords of the projection of 'RayOrigin' onto the closest poly
						EvaluateHit(RayOrigin, ClosestPolyId, SrcMesh, HitPayload);
						HitPayload.Forward = -2;

					}
					else
					{


						const FVector3f VertNormal[3] = { Normal[Indices[Offset + 0]], Normal[Indices[Offset + 1]], Normal[Indices[Offset + 2]] };



						// Look for the closest poly.


						// face normal.. used for testing
						//  FVector3f RayDirection = ComputeNormal(VertPos);

						// The averaged normal at this position.
						FVector3f RayDirection = ProxyLOD::InterpolateVertexData(BarycentricCoords, VertNormal);
						RayDirection.Normalize(); // Todo: what if this normalize fails?



												  // Slight offset to the star point when casting rays to insure that we don't miss the case
												  // when the Src geometry is actually co-planar with triangle we are shooting from.

						const FVector3f ForwardRayStart = RayOrigin - 0.0001 * RayDirection;
						const FVector3f ReverseRayStart = RayOrigin + 0.0001 * RayDirection;

						// We have to shoot rays in forward & backfacing normal direction to look for the closest source triangle. 
						// Construct a forward and a reverse ray.
						FVector3f ForwardRayEnd = ForwardRayStart + RayDirection * MaxRayLength;
						FVector3f ReverseRayEnd = ReverseRayStart - RayDirection * MaxRayLength;

						// Find t

						FkHitResult ForwardResult;

						TkDOPLineCollisionCheck<const FUnitTransformDataProvider, uint32>  ForwardRay((FVector)ForwardRayStart, (FVector)ForwardRayEnd, true, kDOPDataProvider, &ForwardResult);

						FkHitResult ReverseResult;

						TkDOPLineCollisionCheck<const FUnitTransformDataProvider, uint32>  ReverseRay((FVector)ReverseRayStart, (FVector)ReverseRayEnd, true, kDOPDataProvider, &ReverseResult);


						// Fire both rays

						bool bForwardHit = SrckDOPTree.LineCheck(ForwardRay);
						bool bReverseHit = SrckDOPTree.LineCheck(ReverseRay);

						if (bForwardHit && bReverseHit) // Both directions hit.
						{
							ProxyLOD::FSrcMeshData ForwardHitPayload;
							const float ForwardHitNormalDotNormal = EvaluateHit<true>(ForwardResult, ForwardRayStart, RayDirection, SrcMesh, ForwardHitPayload);
							const bool bForwardCodirectional = (ForwardHitNormalDotNormal > 0);

							ProxyLOD::FSrcMeshData ReverseHitPayload;
							const float ReverseHitNormalDotNormal = EvaluateHit<false>(ReverseResult, ReverseRayStart, RayDirection, SrcMesh, ReverseHitPayload);
							const bool bReverseCodirectional = (ReverseHitNormalDotNormal > 0);


							if (bForwardCodirectional && bReverseCodirectional) // Both hits were "valid"
							{
								if (TestTransferType == 0)
								{
									// pick the closest.
									if (ForwardResult.Time < ReverseResult.Time)
									{
										HitPayload = ForwardHitPayload;
									}
									else
									{
										HitPayload = ReverseHitPayload;
									}

								}
								else
								{
									// pick forward
									HitPayload = ForwardHitPayload;
								}
							}
							else if (bForwardCodirectional) // only the forward hit was "valid"
							{
								// pick forward
								HitPayload = ForwardHitPayload;
							}
							else if (bReverseCodirectional) // only the reverse hit was "valid"
							{
								HitPayload = ReverseHitPayload;
							}
							else
							{
								HitPayload = ReverseHitPayload;
								HitPayload.Forward = -1;
							}
						}
						else if (bForwardHit)
						{
							EvaluateHit<true>(ForwardResult, ForwardRayStart, RayDirection, SrcMesh, HitPayload);
						}
						else if (bReverseHit)
						{
							EvaluateHit<false>(ReverseResult, ReverseRayStart, RayDirection, SrcMesh, HitPayload);
						}

					}

					// 
					if (HitPayload.MaterialId != -1)
					{
						TargetTexel = HitPayload;
					}

				}

			}
		}
	});

	return TargetGridPtr;
}


namespace
{

	template <EFlattenMaterialProperties PropertyType>
	void TransferMaterial(const ProxyLOD::FSrcDataGrid& CorrespondenceGrid, const ProxyLOD::FRasterGrid& UVGrid, const TArray<FFlattenMaterial>& InputMaterials, TArray<FLinearColor>& SamplesBuffer, const FLinearColor DefaultColor = FLinearColor(0., 0., 0., 0.))
	{
		checkSlow(CorrespondenceGrid.Size() == UVGrid.Size());

		const FIntPoint Size = UVGrid.Size();
		const int32 BufferSize = Size.X * Size.Y;
		ResizeArray(SamplesBuffer, BufferSize);

		// Init buffer to zero color

		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, BufferSize),
			[&SamplesBuffer, DefaultColor](const ProxyLOD::FIntRange& Range)
		{
			const float MinFloat = std::numeric_limits<float>::lowest();
			FLinearColor* Data = SamplesBuffer.GetData();
			for (int32 i = Range.begin(), I = Range.end(); i < I; ++i) Data[i] = DefaultColor;
		});


		// Iterate over the the UVGrid.

		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, Size.Y),
			[&](const ProxyLOD::FIntRange& Range)
		{

			const FColor EmptySrcColor = FColor::Black; 

			auto GetColorFromBuffer = [](const FIntPoint& Size, const TArray<FColor>& Buffer, const FIntPoint& ij)->FLinearColor
			{
				FIntPoint SampleIJ = ij;
				// if the sample requested is outside the 2d array bounds, just shift it to the closest valid point. 
				if (SampleIJ.X > Size.X - 1) SampleIJ.X = Size.X - 1;
				if (SampleIJ.Y > Size.Y - 1) SampleIJ.Y = Size.Y - 1;
				if (SampleIJ.X < 0) SampleIJ.X = 0;
				if (SampleIJ.Y < 0) SampleIJ.Y = 0;

				FColor SampleColor = Buffer[SampleIJ.X + SampleIJ.Y * Size.X];

				return FLinearColor(SampleColor);
			};

			for (int32 j = Range.begin(); j < Range.end(); ++j)
			{

				for (int32 i = 0; i < Size.X; ++i)
				{

					const float MinFloat = std::numeric_limits<float>::lowest();

					// init the result with a non-valid number.  
					FLinearColor ResultLinearColor = FLinearColor(MinFloat, 0.f, 0.f, 0.f);

					const auto& DstTexelData = UVGrid(i, j);
					const int32 DstTriangleId = DstTexelData.TriangleId;
					const ProxyLOD::FSrcMeshData& CorrespondenceData = CorrespondenceGrid(i, j);

					if (DstTriangleId > -1)
					{

						if (CorrespondenceData.MaterialId == -1) continue;

						const int32 MaterialIdx = CorrespondenceData.MaterialId;

						FVector2D SrcUV = CorrespondenceData.UV;

						// Why is this happening?
						if (SrcUV.X > 1 || SrcUV.X < 0 || SrcUV.Y > 1 || SrcUV.Y < 0)
						{
							SrcUV = FVector2D(0.f, 0.f);
						}


						const FIntPoint SrcBufferSize = InputMaterials[MaterialIdx].GetPropertySize(PropertyType);
						const TArray<FColor>& SrcBuffer = InputMaterials[MaterialIdx].GetPropertySamples(PropertyType);

						// Look up the color at the hit point


						// bilinear interpolation of the color
						if (SrcBuffer.Num() > 1)
						{
							const FVector2D SrcTexelLocation(SrcUV.X * SrcBufferSize.X, SrcUV.Y * SrcBufferSize.Y);

							// 0,0 corner
							const FIntPoint Min(FMath::FloorToInt32(SrcTexelLocation.X), FMath::FloorToInt32(SrcTexelLocation.Y));

							// offset.
							const FVector2D Delta(SrcTexelLocation.X - (float)Min.X, SrcTexelLocation.Y - (float)Min.Y);

							// corner samples.
							FLinearColor Samples[4];
							Samples[0] = GetColorFromBuffer(SrcBufferSize, SrcBuffer, Min + FIntPoint(0, 0));
							Samples[1] = GetColorFromBuffer(SrcBufferSize, SrcBuffer, Min + FIntPoint(1, 0));
							Samples[2] = GetColorFromBuffer(SrcBufferSize, SrcBuffer, Min + FIntPoint(0, 1));
							Samples[3] = GetColorFromBuffer(SrcBufferSize, SrcBuffer, Min + FIntPoint(1, 1));

							FLinearColor BilinearInterp =
								Samples[0] * (1. - Delta.X) * (1. - Delta.Y) +
								Samples[1] * Delta.X * (1. - Delta.Y) +
								Samples[2] * (1. - Delta.X) * Delta.Y +
								Samples[3] * Delta.X * Delta.Y;

							// The interpolated color
							ResultLinearColor = BilinearInterp;
						}
						else if (SrcBuffer.Num() == 1)
						{
							//ResultColor = SrcBuffer[0];
							ResultLinearColor = FLinearColor(SrcBuffer[0]);

						}
						else // SrcBuffer.Num() == 0
						{
							//ResultColor = EmptySrcColor;
							ResultLinearColor = FLinearColor(EmptySrcColor);

						}
					}

					// Store the result.

					SamplesBuffer[i + j * Size.X] = ResultLinearColor;

				}
			}
		});
	}


	/**
	*  Down sample a grid by area-weighted averaging: Each data point ( i.e. Value = Grid(i,j) ) is assumed to represent
	*  a uniform value within the enclosing grid cell [i, j] x [i+1, j+1].
	*
	*  NB: The size (in each direction) of the OutGrid must be smaller than the InSrcGrid
	*
	* @param InSrcGrid  The original grid to be down sampled.
	* @param OutGrid    The result grid.  Any data already in this grid is lost.
	*
	*/
	void DownSample2dGrid(const ProxyLOD::FLinearColorGrid& InSrcGrid, ProxyLOD::TGrid<FLinearColor>& OutGrid)
	{
		typedef FLinearColor   ValueType;

		const FIntPoint SrcSize = InSrcGrid.Size();
		const FIntPoint DstSize = OutGrid.Size();

		checkSlow(SrcSize.X >= DstSize.X);
		checkSlow(SrcSize.Y >= DstSize.Y);

		// assign each scanline to a different parallel task
		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, DstSize.Y), [&InSrcGrid, &OutGrid](const ProxyLOD::FIntRange& Range)
		{
			const int32 I = OutGrid.Size().X;

			const int32 SrcI = InSrcGrid.Size().X;
			const int32 SrcJ = InSrcGrid.Size().Y;

			double Dx = 1. / double(I);
			double Dy = 1. / double(OutGrid.Size().Y);

			double SrcDx = 1. / double(SrcI);
			double SrcDy = 1. / double(SrcJ);



			for (int32 j = Range.begin(), J = Range.end(); j < J; ++j)
			{

				// loop over the 'fast' grid direction, gathering from the src grid.
				for (int32 i = 0; i < I; ++i)
				{
					// Destination for our data is a 
					// small square centered on i+1/2, j+1/2
					// in unit space (US) 

					double DstUSMin[2] = { Dx * i,       Dy * j };
					double DstUSMax[2] = { Dx * (i + 1), Dy * (j + 1) };

					// Where are these corners in the SrcGrid 
					// in the grid index space (IS) ?
					int32 SrcISMin[2] = { FMath::FloorToInt32(DstUSMin[0] * SrcI), FMath::FloorToInt32(DstUSMin[1] * SrcJ) };
					int32 SrcISMax[2] = { FMath::CeilToInt32(DstUSMax[0] * SrcI), FMath::CeilToInt32(DstUSMax[1] * SrcJ) };

					// clip against boundary
					SrcISMin[0] = FMath::Max(SrcISMin[0], 0);
					SrcISMin[1] = FMath::Max(SrcISMin[1], 0);

					SrcISMax[0] = FMath::Min(SrcISMax[0], SrcI);
					SrcISMax[1] = FMath::Min(SrcISMax[1], SrcJ);

					// access to the value we are going to update
					ValueType&  DstValue = OutGrid(i, j);

					DstValue = ValueType(ForceInitToZero);

					double TotalIntersectArea = 0.;
					// loop over the src region that contributes to the dst region
					for (int32 jsrc = SrcISMin[1]; jsrc < SrcISMax[1]; ++jsrc)
					{
						for (int32 isrc = SrcISMin[0]; isrc < SrcISMax[0]; ++isrc)
						{
							const ValueType SrcValue = InSrcGrid(isrc, jsrc);

							// the bounding box in unit space for this src grid cell
							double SrcUSMin[2] = { SrcDx * isrc,       SrcDy * jsrc };
							double SrcUSMax[2] = { SrcDx * (isrc + 1), SrcDy * (jsrc + 1) };

							// compute the intersection of the Src grid cell with the Dst grid cell.
							double IntersectMin[2] = { FMath::Max(SrcUSMin[0], DstUSMin[0]), FMath::Max(SrcUSMin[1], DstUSMin[1]) };
							double IntersectMax[2] = { FMath::Min(SrcUSMax[0], DstUSMax[0]), FMath::Min(SrcUSMax[1], DstUSMax[1]) };

							// area of intersection
							double IntersectArea = (IntersectMax[0] - IntersectMin[0]) * (IntersectMax[1] - IntersectMin[1]);

							TotalIntersectArea += IntersectArea;
							DstValue += IntersectArea * SrcValue;
						}

					}

					if (TotalIntersectArea > .1 * Dx * Dy) // less that 1/10 of the target texel
					{
						DstValue = DstValue / TotalIntersectArea;
					}
					else
					{
						DstValue = ValueType(ForceInitToZero);
					}

				}

			}
		});
	}

	/**
	*	Sparse Down Sample of Color (FLinearColor) Grid. 
	*   Specialized color down-sample that also preserves average luma (computed with with Rec 709 standard)
	*
	*  @param SuperSampleGrid        Array of FLinearColor data in a high resolution grid.
	*                                The sparsity corresponds to Charts in a UV atlas and regions outside of
	*                                the charts hold non-physical colors (std::numeric_limits<float>::lowest())
	*
	*  @param OutGrid                Resulting grid holding the normal map texture data as FLinearColor
	*
	*  NB: It is assumed that the SuperSample Grid is a multiple of the OutBuffer Grid.  I.e. an integer number of super sample
	*      texels correspond to each result texel
	*/
	void SparseDownSampleColor(const ProxyLOD::TGridWrapper<FLinearColor>& SuperSampleGrid, ProxyLOD::FLinearColorGrid& OutGrid, const FLinearColor DefaultColor = FLinearColor(0., 0., 0., 0.))
	{
		FIntPoint SuperSampleSize = SuperSampleGrid.Size();
		FIntPoint ResultSize      = OutGrid.Size();
		const int32 NumXSuperSamples = SuperSampleSize.X / ResultSize.X;
		const int32 NumYSuperSamples = SuperSampleSize.Y / ResultSize.Y;

		checkSlow(SuperSampleSize.X == ResultSize.X * NumXSuperSamples);
		checkSlow(SuperSampleSize.Y == ResultSize.Y * NumYSuperSamples);

		// Note FLinearColor has a different way to define Luma.
		// the Rec 709 standard is dot(Color, float3(0.2126, 0.7152, 0.0722));
		const auto ComputeLuma = [](const FLinearColor& Color)->float
		{
			bool bUseRec709 = false;
			float Result;

			if (bUseRec709)
			{
				Result = Color.R * 0.2126 + Color.G * 0.7152 + Color.B  * 0.0722;
			}
			else
			{
				Result = Color.GetLuminance();
			}

			return Result;
		};

		// Gather
		
		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, ResultSize.Y),
			[&](const ProxyLOD::FIntRange& Range)
		{
			const FLinearColor* SuperSampleBuffer = SuperSampleGrid.GetData();
			FLinearColor* ResultBuffer = OutGrid.GetData();
			for (int32 j = Range.begin(), J = Range.end(); j < J; ++j)
			{
				int32 joffset = j * NumYSuperSamples;
				for (int32 i = 0; i < ResultSize.X; ++i)
				{

					FLinearColor ResultColor(0, 0, 0, 0);
					float        Luma = 0;
					uint32       ResultCount = 0;

					int32 ioffset = i * NumXSuperSamples;

					// loop over the super samples
					for (int32 jj = joffset; jj < joffset + NumYSuperSamples; ++jj)
					{
						for (int32 ii = ioffset; ii < ioffset + NumXSuperSamples; ++ii)
						{
							const FLinearColor& SuperSampleColor = SuperSampleBuffer[ii + jj * SuperSampleSize.X];
							if (SuperSampleColor.R != std::numeric_limits<float>::lowest())
							{
								ResultColor += SuperSampleColor;

								Luma += ComputeLuma(SuperSampleColor);
								ResultCount++;
							}

						}
					}


					// Average the result
					if (ResultCount > 0)
					{
						ResultColor = ResultColor / float(ResultCount);
						Luma = Luma / float(ResultCount);
						float TmpLuma = ComputeLuma(ResultColor);
						if (TmpLuma > 1.e-3)
						{
							float LumaCorrection = Luma / ComputeLuma(ResultColor);
							ResultColor *= LumaCorrection;
						}
					}
					else
					{
						ResultColor = DefaultColor;
					}

					// write it into the buffer
					ResultBuffer[i + j * ResultSize.X] = ResultColor;
				}
			}
		});
	}

	/**
	*	Sparse Down Sample of FLinearColor Grid.
	*
	*
	*  @param SuperSampledBuffer     Array of FLinearColor data in a high resolution grid.
	*                                The sparsity corresponds to Charts in a UV atlas and regions outside of
	*                                the charts hold non-physical colors (std::numeric_limits<float>::lowest())
	*
	*  @param OutGrid                Resulting grid holding the normal map texture data as FLinearColor
	*
	*  NB: It is assumed that the SuperSample Grid is a multiple of the OutBuffer Grid.  I.e. an interger number of super sample
	*      texels correspond to each result texel
	*/
	void SparseDownSampleMaterial(const ProxyLOD::TGridWrapper<FLinearColor>& SuperSampleGrid, ProxyLOD::FLinearColorGrid& OutGrid)
	{
		const FIntPoint  SuperSampleSize = SuperSampleGrid.Size();
		const FIntPoint  ResultSize      = OutGrid.Size();
		const int32 NumXSuperSamples = SuperSampleSize.X / ResultSize.X;
		const int32 NumYSuperSamples = SuperSampleSize.Y / ResultSize.Y;

		checkSlow(SuperSampleSize.X == ResultSize.X * NumXSuperSamples);
		checkSlow(SuperSampleSize.Y == ResultSize.Y * NumYSuperSamples);

		// Gather

		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, ResultSize.Y),
			[&](const ProxyLOD::FIntRange& Range)
		{
			const FLinearColor* SuperSampleBuffer = SuperSampleGrid.GetData();
			FLinearColor* ResultBuffer = OutGrid.GetData();
			for (int32 j = Range.begin(), J = Range.end(); j < J; ++j)
			{
				int32 joffset = j * NumYSuperSamples;
				for (int32 i = 0; i < ResultSize.X; ++i)
				{

					FLinearColor ResultColor(0, 0, 0, 0);
					uint32       ResultCount = 0;

					int32 ioffset = i * NumXSuperSamples;

					// loop over the super samples
					for (int32 jj = joffset; jj < joffset + NumYSuperSamples; ++jj)
					{
						for (int32 ii = ioffset; ii < ioffset + NumXSuperSamples; ++ii)
						{
							const FLinearColor& SuperSampleColor = SuperSampleBuffer[ii + jj * SuperSampleSize.X];
							if (SuperSampleColor.R != std::numeric_limits<float>::lowest())
							{
								ResultColor += SuperSampleColor;
								ResultCount++;
							}

						}
					}


					// Average the result
					if (ResultCount > 0) ResultColor = ResultColor / float(ResultCount);

					// write it into the buffer
					ResultBuffer[i + j * ResultSize.X] = ResultColor;
				}
			}
		});
	}

	// Unpack a quantized vector into the [-1,1]x[-1,1]x[-1,1] cube
	FVector3f UnPackQuantizedVector(const FColor& QuantizedNormal)
	{
		FVector3f Vector(QuantizedNormal.R / 255.f, QuantizedNormal.G / 255.f, QuantizedNormal.B / 255.f);

		Vector = 2.f * Vector - FVector3f(1., 1., 1.);
		return Vector;
	}

	FColor QuantizeVector(const FVector3f& Vector)
	{
		FVector3f Tmp = 0.5 * (Vector + FVector3f(1., 1., 1.));
		FLinearColor LinearColor(Tmp);
		return LinearColor.QuantizeRound();
	}

	// Specialized version for the normal map. 
	// This needs to convert the normal vector to the tangent space of the target mesh.
	// In the case that the source geometry didn't have a normal map, we have to generate one by 
	// directly sampling the geometry normals.

	void SuperSampleWSNormal(const FMeshDescriptionArrayAdapter& SrcMeshAdapter,
		const ProxyLOD::FSrcDataGrid& SuperSampleCorrespondenceGrid,
		const ProxyLOD::FRasterGrid& SuperSampleUVGrid,
		const TArray<FFlattenMaterial>& InputMaterials,
		TArray<FVector3f>& SuperSampleBuffer)
	{

		typedef FVector3f FTangentSpace[3];

		//	const FIntPoint Size = OutMaterial.GetPropertySize(PropertyType);
		//	TArray<FColor>& TargetBuffer = OutMaterial.GetPropertySamples(PropertyType);
		const FIntPoint  BufferSize = SuperSampleUVGrid.Size();
		int32 BufferCount = BufferSize.X * BufferSize.Y;
		ResizeArray(SuperSampleBuffer, BufferCount);

		// Init buffer to zero color

		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, BufferCount),
			[&SuperSampleBuffer](const ProxyLOD::FIntRange& Range)
		{
			FVector3f* Data = SuperSampleBuffer.GetData();
			for (int32 i = Range.begin(), I = Range.end(); i < I; ++i) Data[i] = FVector3f(0.f, 0.f, 0.f);
		});


		checkSlow(SuperSampleCorrespondenceGrid.Size() == SuperSampleUVGrid.Size());

		const FColor DefaultMissColor(255, 0, 0);
		const FColor EmptySrcColor(0, 0, 255);

		// Iterate over the the UVGrid.

		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, BufferSize.Y),
			[&](const ProxyLOD::FIntRange& Range)
		{

			auto ComputeSrcTangentSpace = [&SrcMeshAdapter](const int32 TriangleId, const ProxyLOD::DArray3d& BarycentricCoords, FTangentSpace& TangentSpace)
			{
				int32 MeshId, InstanceIdx, LocalFaceNumber;
				const FMeshDescriptionArrayAdapter::FRawPoly SrcPoly = SrcMeshAdapter.GetRawPoly(TriangleId, MeshId, InstanceIdx, LocalFaceNumber, ERawPolyValues::WedgeTangents);

				// Get the tangent space:  We try to compute this in a way that is consistent with LocalVertexFactory.usf (CalcTangentToLocal).

				TangentSpace[0] = ProxyLOD::InterpolateVertexData(BarycentricCoords, SrcPoly.WedgeTangentX);
				//TangentSpace[1] = ProxyLOD::InterpolateVertexData(BarycentricCoords, SrcPoly.WedgeTangentY);
				TangentSpace[2] = ProxyLOD::InterpolateVertexData(BarycentricCoords, SrcPoly.WedgeTangentZ);

				TangentSpace[2].Normalize();
				TangentSpace[0].Normalize();

				// bi-tangent
				TangentSpace[1] = FVector3f::CrossProduct(TangentSpace[2], TangentSpace[0]);

				// tangent
				TangentSpace[0] = FVector3f::CrossProduct(TangentSpace[1], TangentSpace[2]);

			};



			// Samples the normal map and unpacks the result
			auto GetNormalFromBuffer = [](const FIntPoint& Size, const TArray<FColor>& Buffer, const FIntPoint& ij)->FVector3f
			{
				FIntPoint SampleIJ = ij;
				// if the sample requested is outside the 2d array bounds, just shift it to the closest valid point. 
				if (SampleIJ.X > Size.X - 1) SampleIJ.X = Size.X - 1;
				if (SampleIJ.Y > Size.Y - 1) SampleIJ.Y = Size.Y - 1;
				if (SampleIJ.X < 0) SampleIJ.X = 0;
				if (SampleIJ.Y < 0) SampleIJ.Y = 0;

				// TangentSpace Normal
				FColor NormalAsColor = Buffer[SampleIJ.X + SampleIJ.Y * Size.X];

				// Map from uint8 0, 255 x 0,255 x 0,255  to -1,1 x -1,1 x -1,1 region
				FVector3f TSNormal = UnPackQuantizedVector(NormalAsColor);

				TSNormal.Normalize();
				return TSNormal;

			};

			// Iterate over the target texels.

			for (int32 j = Range.begin(); j < Range.end(); ++j)
			{

				for (int32 i = 0; i < BufferSize.X; ++i)
				{


					const float MinFloat = std::numeric_limits<float>::lowest();
					FVector3f& ResultNormal = SuperSampleBuffer[i + j * BufferSize.X];
					ResultNormal = FVector3f(MinFloat, 0.f, 0.f);


					const auto& SuperSampleTexelData = SuperSampleUVGrid(i, j);
					const ProxyLOD::FSrcMeshData& CorrespondenceData = SuperSampleCorrespondenceGrid(i, j);
					const int32 DstTriangleId = SuperSampleTexelData.TriangleId;

					if (DstTriangleId > -1)
					{
						const auto& DstBarycentricCoords = SuperSampleTexelData.BarycentricCoords;



						if (CorrespondenceData.MaterialId == -1) continue;

						const int32 MaterialIdx = CorrespondenceData.MaterialId;

						FVector2D SrcUV = CorrespondenceData.UV;

						const int32 SrcTriangleId = CorrespondenceData.TriangleId;
						const auto& SrcBarycentricCoords = CorrespondenceData.BarycentricCoords;
						FTangentSpace SrcTangentSpace;
						ComputeSrcTangentSpace(SrcTriangleId, SrcBarycentricCoords, SrcTangentSpace);
						
						// Correct for triangles with inverted chirality (inverted UV handedness )
						SrcTangentSpace[1] *= CorrespondenceData.Handedness;


						// Why is this happening?
						if (SrcUV.X > 1 || SrcUV.X < 0 || SrcUV.Y > 1 || SrcUV.Y < 0)
						{
							SrcUV = FVector2D(0.f, 0.f);
						}


						const FIntPoint SrcBufferSize = InputMaterials[MaterialIdx].GetPropertySize(EFlattenMaterialProperties::Normal);
						const TArray<FColor>& SrcBuffer = InputMaterials[MaterialIdx].GetPropertySamples(EFlattenMaterialProperties::Normal);


						// By convention, a baked down src normal map will have a single element in the case that the original geoemetry had no normal map.

						const bool bSrcHasNormalMap = (SrcBuffer.Num() > 1);


						// Get world space samples of the normal - either from the normal map or directly from geometry.
						//  Sample the normal map in world space.  Use bilinear filtering.

						if (bSrcHasNormalMap) // testing.  We should just be encoding the geometry normal now.
						{

							const FVector2D SrcTexelLocation(SrcUV.X * SrcBufferSize.X, SrcUV.Y * SrcBufferSize.Y);

							// 0,0 corner

							const FIntPoint Min(FMath::FloorToInt32(SrcTexelLocation.X), FMath::FloorToInt32(SrcTexelLocation.Y));

							// offset.

							const FVector2D Delta(SrcTexelLocation.X - (float)Min.X, SrcTexelLocation.Y - (float)Min.Y);

							// corner samples.

							FVector3f NormalSampleArray[4];
							NormalSampleArray[0] = GetNormalFromBuffer(SrcBufferSize, SrcBuffer, Min + FIntPoint(0, 0));
							NormalSampleArray[1] = GetNormalFromBuffer(SrcBufferSize, SrcBuffer, Min + FIntPoint(1, 0));
							NormalSampleArray[2] = GetNormalFromBuffer(SrcBufferSize, SrcBuffer, Min + FIntPoint(0, 1));
							NormalSampleArray[3] = GetNormalFromBuffer(SrcBufferSize, SrcBuffer, Min + FIntPoint(1, 1));

							// Interpolate the results

							const FVector3f TangentSpaceNormal =
								NormalSampleArray[0] * (1. - Delta.X) * (1. - Delta.Y) +
								NormalSampleArray[1] * Delta.X * (1. - Delta.Y) +
								NormalSampleArray[2] * (1. - Delta.X) * Delta.Y +
								NormalSampleArray[3] * Delta.X * Delta.Y;



							// Convert to worldspace 

							ResultNormal = TangentSpaceNormal.X * SrcTangentSpace[0] + TangentSpaceNormal.Y * SrcTangentSpace[1] + TangentSpaceNormal.Z * SrcTangentSpace[2];

							// If the src triangle we are sampling actually faces the opposite direction of the dst triangle, we assume the src poly was marked as double sided.

							//	ResultNormal[2] *= CorrespondenceData.CoAligned;

						}
						else // Src doesn't have NormalMap:   Sample the geometry normal in world space.
						{
							// We aren't filtering the normal from the geometry, but the fact that we 
							// are doing super-sampling should help average things out.

							ResultNormal = SrcTangentSpace[2];
						}


					}	// end if in triangle.					

				} // end i loop
			} // end j loop
		});
	}

	/**
	*	Sparse Down Sample of Normal Vector. 
	*
	*   NB: Converts a world space normal vector to the tangent space of the Destination Mesh.
	*       Also the resulting tangent vector is encoded with a shift and scale 1/2( tangent + {1,1,1})
	*       So it make be stored as an FLinearColor
	*
	*  @param SuperSampledNormals    Array of Normals stored sparsely in World Space in a high resolution grid.
	*                                The sparsity corresponds to Charts in a UV atlas and regions outside of
	*                                the charts hold non-physical normals (std::numeric_limits<float>::lowest())
	*
	*  @param DstUVGrid              Grid of the same size as the output that maps texel in the UV Atlas 
	*                                to points on corresponding DstRawMesh.
	*                            
	*  @param DstRawMesh             Mesh (with tangent space and UVs) for which we are encoding normals
	*
	*  @param OutDstBufferGrid       Resulting grid holding the normal map texture data as FLinearColor
	*
	*
	*  NB: It is assumed that the SuperSample Grid is a multiple of the OutBuffer Grid.  I.e. an integer number of super sample
	*      texels correspond to each result texel
	*  
	*/
	void SparseDownSampleNormal(const ProxyLOD::TGridWrapper<FVector3f>& SuperSampledNormalGrid,
		const ProxyLOD::FRasterGrid& DstUVGrid, const FMeshDescription& MeshDescription,
		ProxyLOD::FLinearColorGrid& OutDstBufferGrid)
	{
		FStaticMeshConstAttributes Attributes(MeshDescription);

		TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
		TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();

		const FIntPoint SuperSampleSize = SuperSampledNormalGrid.Size();
		const FIntPoint DstBufferSize = OutDstBufferGrid.Size();
		const FIntPoint UVSize = DstUVGrid.Size();

		const int32 NumXSuperSamples = SuperSampleSize.X / UVSize.X;
		const int32 NumYSuperSamples = SuperSampleSize.Y / UVSize.Y;


		checkSlow(SuperSampleSize.X == UVSize.X * NumXSuperSamples);
		checkSlow(SuperSampleSize.Y == UVSize.Y * NumYSuperSamples);


		// Gather

		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, DstBufferSize.Y),
			[&](const ProxyLOD::FIntRange& Range)
		{
			// Store the tangent space in a 3x3 matrix.  

			typedef openvdb::Mat3R  FTangentSpace;


			auto ComputeDstTangentSpace = [&MeshDescription, &VertexInstanceTangents, &VertexInstanceBinormalSigns, &VertexInstanceNormals](const int32 TriangleId, const ProxyLOD::DArray3d& BarycentricCoords, FTangentSpace& TangentSpace)
			{
				const FVertexInstanceID VertexInstanceIDs[3] = { FVertexInstanceID(3 * TriangleId),
																 FVertexInstanceID(3 * TriangleId + 1),
																 FVertexInstanceID(3 * TriangleId + 2) };


				// Compute the local tangent space
				FVector3f InterpolatedTangent;
				FVector3f TangentSamples[3];
				TangentSamples[0] = VertexInstanceTangents[VertexInstanceIDs[0]];
				TangentSamples[1] = VertexInstanceTangents[VertexInstanceIDs[1]];
				TangentSamples[2] = VertexInstanceTangents[VertexInstanceIDs[2]];
				InterpolatedTangent = ProxyLOD::InterpolateVertexData(BarycentricCoords, TangentSamples);

				FVector3f InterpolatedBiTangent;
				// The bi-tangent is reconstructed from the tangent and normal
				//{
				FVector3f BiTangentSamples[3];
				BiTangentSamples[0] = FVector3f::CrossProduct(VertexInstanceNormals[VertexInstanceIDs[0]], VertexInstanceTangents[VertexInstanceIDs[0]]).GetSafeNormal() * VertexInstanceBinormalSigns[VertexInstanceIDs[0]];
				BiTangentSamples[1] = FVector3f::CrossProduct(VertexInstanceNormals[VertexInstanceIDs[1]], VertexInstanceTangents[VertexInstanceIDs[1]]).GetSafeNormal() * VertexInstanceBinormalSigns[VertexInstanceIDs[1]];
				BiTangentSamples[2] = FVector3f::CrossProduct(VertexInstanceNormals[VertexInstanceIDs[2]], VertexInstanceTangents[VertexInstanceIDs[2]]).GetSafeNormal() * VertexInstanceBinormalSigns[VertexInstanceIDs[2]];
				InterpolatedBiTangent = ProxyLOD::InterpolateVertexData(BarycentricCoords, BiTangentSamples);
				//}

				FVector3f InterpolatedNormal;
				//{
				FVector3f NormalSamples[3];
				NormalSamples[0] = VertexInstanceNormals[VertexInstanceIDs[0]];
				NormalSamples[1] = VertexInstanceNormals[VertexInstanceIDs[1]];
				NormalSamples[2] = VertexInstanceNormals[VertexInstanceIDs[2]];
				InterpolatedNormal = ProxyLOD::InterpolateVertexData(BarycentricCoords, NormalSamples);
				//}

				// Testing handedness. Is the tangent x bitangent pointing (roughly) in the direction of the normal?
				float Chirality[3];
				for (int32 i = 0; i < 3; ++i)
				{
					// StaticMeshVertexBuffer.h computes it this way...(see SetTangents)
					FMatrix Basis(
						FPlane((FVector)TangentSamples[i], 0),
						FPlane((FVector)BiTangentSamples[i], 0),
						FPlane((FVector)NormalSamples[i], 0),
						FPlane(0, 0, 0, 1)
					);
					Chirality[i] = (Basis.Determinant() < 0) ? -1.f : 1.f;
				}
				// NB: I don't know why this minus sign makes things work.. it shouldn't..
				float Handedness = FMath::Clamp(ProxyLOD::InterpolateVertexData(BarycentricCoords, Chirality), -1.f, 1.f);
				Handedness = (Handedness > 0) ? 1.f : -1.f;
	

				InterpolatedBiTangent = Handedness * FVector3f::CrossProduct(InterpolatedNormal, InterpolatedTangent);
				InterpolatedTangent = Handedness * FVector3f::CrossProduct(InterpolatedBiTangent, InterpolatedNormal);



				// Put the result in the column of a 3x3 matrix
				TangentSpace.setColumns(openvdb::Vec3f(InterpolatedTangent.X,   InterpolatedTangent.Y,   InterpolatedTangent.Z),
										openvdb::Vec3f(InterpolatedBiTangent.X, InterpolatedBiTangent.Y, InterpolatedBiTangent.Z),
										openvdb::Vec3f(InterpolatedNormal.X,    InterpolatedNormal.Y,    InterpolatedNormal.Z));

			};

			// Dotproduct between Vec3f and FVector3f

			auto LocalDot = [](const openvdb::Vec3f& VecA, const FVector3f& VecB)->float
			{
				return VecA[0] * VecB[0] + VecA[1] * VecB[1] + VecA[2] * VecB[2];
			};

			// Loop over the texels
			FLinearColor* DstBuffer            = OutDstBufferGrid.GetData();
			const FVector3f* SuperSampledNormals = SuperSampledNormalGrid.GetData();

			for (int32 j = Range.begin(), J = Range.end(); j < J; ++j)
			{
				int32 joffset = j * NumYSuperSamples;
				for (int32 i = 0; i < DstBufferSize.X; ++i)
				{
					
					FLinearColor& ResultColor = DstBuffer[i + j * DstBufferSize.X];
					ResultColor = FLinearColor(0, 0, 0, 0);

					// The actual requested texture size might be smaller than the buffer we are filling.

					if (i > UVSize.X - 1 || j > UVSize.Y - 1) continue;

					// Encode this normal in terms of the local tangent space.
					const auto& DstTexel = DstUVGrid(i, j);

					if (DstTexel.TriangleId < 0) continue;

					FTangentSpace XForm;
					ComputeDstTangentSpace(DstTexel.TriangleId, DstTexel.BarycentricCoords, XForm);

					FVector3f      ResultVector(0, 0, 0);
					uint32       ResultCount = 0;

					int32 ioffset = i * NumXSuperSamples;

					const openvdb::Vec3f WorldNormal = XForm.col(2);
					
					// loop over the super samples
					for (int32 jj = joffset; jj < joffset + NumYSuperSamples; ++jj)
					{
						for (int32 ii = ioffset; ii < ioffset + NumXSuperSamples; ++ii)
						{
							const FVector3f& SuperSampleColor = SuperSampledNormals[ii + jj * SuperSampleSize.X];
							if (SuperSampleColor.X != std::numeric_limits<float>::lowest())
							{
								const float DotWithWorldNormal = LocalDot(WorldNormal, SuperSampleColor);
								ResultVector += SuperSampleColor / (DotWithWorldNormal * DotWithWorldNormal + 0.1);
								ResultCount++;
							}

						}
					}
					// Average the result
					if (ResultCount > 0)
					{
						ResultVector = ResultVector / float(ResultCount);
					}

					
					// Invert the tangent space
					if (FMath::Abs(XForm.det()) > 0.001)
					{
						XForm = XForm.inverse();
					}
					else
					{
						XForm = XForm.transpose();
					}


					// project the result onto the tangent space vectors.
					// note: the matrix formulation doesn't assume that the local tangent space is orthonormal.

					openvdb::Vec3f Tmp(ResultVector.X, ResultVector.Y, ResultVector.Z);

					Tmp = XForm * Tmp;
					
					// if outside of the compression range, then normalize
					if (Tmp[0] < -1.f || Tmp[0] > 1.f || Tmp[1] < -1.f || Tmp[1] > 1.f || Tmp[2] < -1.f || Tmp[2] > 1.f)
					{
						Tmp.normalize();
					}

					// shift to the 0, range.
					Tmp = 0.5f * (Tmp + openvdb::Vec3f(1, 1, 1));

					// write it into the buffer
					ResultColor = FLinearColor(Tmp[0], Tmp[1], Tmp[2]);
					
				}
			}
		});

	}

	/**
	* Specialized function that assumes the FLinearColor data in the Vector grid
	* represents and scaled and shifted 3-vector.  FLinearColor data = 1/2 (v + {1,1,1})
	* where 'v' is the vector of interest.
	*
	* On return: The grid will hold scaled and shifted normalized vectors.
	*/
	void NormalizeLinearColorVectorGrid(ProxyLOD::FLinearColorGrid& VectorGrid)
	{
		const FIntPoint GridSize = VectorGrid.Size();
		FLinearColor* ColorData = VectorGrid.GetData();

		// renormalize the values after the filtering of the down sample
		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, GridSize.X * GridSize.Y),
			[ColorData](const ProxyLOD::FIntRange& Range)
		{

			for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				FLinearColor& ColorValue = ColorData[i];
				// convert back to vector form
				openvdb::Vec3f NormVec(ColorValue.R, ColorValue.G, ColorValue.B);
				NormVec = 2.f * NormVec - openvdb::Vec3f(1.f, 1.f, 1.f);
				// normalize
				NormVec.normalize();
				// back to linear color form
				NormVec = 0.5f * (NormVec + openvdb::Vec3f(1.f, 1.f, 1.f));

				ColorValue.R = NormVec[0];
				ColorValue.G = NormVec[1];
				ColorValue.B = NormVec[2];
			}

		});
	}

	/**
	*  Use the information in the super sample grid to identify the texels they contribute to in the regular (not super sampled) grid.
	*  These are grid cells are marked with the value '1',
	*  if no super sample cells contribute to a target cell it will be marked with a '0'
	*/
	void ConstrucTopologyStencilGrid(const ProxyLOD::FRasterGrid& SuperSampledDstUVGrid, ProxyLOD::TGrid<int32>& TopologyGrid)
	{
		const FIntPoint TargetSize = TopologyGrid.Size();
		const FIntPoint SrcSize    = SuperSampledDstUVGrid.Size();


		const int32 SampleCount = SrcSize.X / TargetSize.X;

		checkSlow(SampleCount == SrcSize.Y / TargetSize.Y);

		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, TargetSize.Y),
			[TargetSize, &TopologyGrid, &SuperSampledDstUVGrid, SampleCount](const ProxyLOD::FIntRange& Range)
		{

			for (int32 j = Range.begin(); j < Range.end(); ++j)
			{
				for (int32 i = 0; i < TargetSize.X; ++i)
				{
					// will a triangle contribute to this super sampled texel?
					int32 ResultCounter = 0;

					for (int32 ii = i * SampleCount; ii < (i + 1) * SampleCount; ++ii)
					{
						for (int32 jj = j * SampleCount; jj < (j + 1) * SampleCount; ++jj)
						{
							const auto& TexelData = SuperSampledDstUVGrid(ii, jj);

							if (TexelData.TriangleId > -1) ResultCounter = 1;
						}
					}
					TopologyGrid(i, j) = ResultCounter;
				}
			}
		});
	}


	template <EFlattenMaterialProperties PropertyType>
	void TMapFlattenMaterial(const FMeshDescription& DstRawMesh, const FMeshDescriptionArrayAdapter& SrcMeshAdapter,
		const ProxyLOD::FSrcDataGrid&  SuperSampledCorrespondenceGrid,
		const ProxyLOD::FRasterGrid& SuperSampledDstUVGrid, const ProxyLOD::FRasterGrid& DstUVGrid,
		const TArray<FFlattenMaterial>& InputMaterials, FFlattenMaterial& OutMaterial, const FColor DefaultColor = FColor::Black)
	{
		const FIntPoint OutSize = OutMaterial.GetPropertySize(PropertyType);

		// Transfer the material to a buffer that matches the resolution of our UV grid.
		// Later this is down-sampled for the output material.

		const FIntPoint TransferBufferSize = DstUVGrid.Size();

		// Early out if no dst size has been allocated

		if (TransferBufferSize == FIntPoint::ZeroValue || OutSize == FIntPoint::ZeroValue)
		{
			return;
		}
   
		const int32 SampleCount = SuperSampledDstUVGrid.Size().X / DstUVGrid.Size().X;

		checkSlow(SampleCount == SuperSampledDstUVGrid.Size().Y / DstUVGrid.Size().Y);
		checkSlow(SuperSampledCorrespondenceGrid.Size() == SuperSampledDstUVGrid.Size());

		{

			// Sample into a linear color buffer.
			// Note, only the normal sampler uses the HighRes (MeshAdapter) and Simplified (VertexDataMesh) 
			// geometry. 

			ProxyLOD::FLinearColorGrid::Ptr LinearColorGrid = ProxyLOD::FLinearColorGrid::Create(TransferBufferSize.X, TransferBufferSize.Y);

			if (PropertyType == EFlattenMaterialProperties::Normal)
			{

				TArray<FVector3f> SuperSampleBuffer;
				SuperSampleWSNormal(SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, InputMaterials, SuperSampleBuffer);

				// Respect the local tangent spaces when down sampling the normal.
				ProxyLOD::TGridWrapper<FVector3f> SuperSampleGrid(SuperSampleBuffer, SuperSampledDstUVGrid.Size());
				SparseDownSampleNormal(SuperSampleGrid, DstUVGrid, DstRawMesh, *LinearColorGrid);
			}
			else if (PropertyType == EFlattenMaterialProperties::Diffuse)
			{
				// The base color to use when ray-based transfer can't find any source geometry.  This happens most when
				// using gap filling to close doors and windows.
				const FLinearColor UnresolvedColor(DefaultColor);
				
				TArray<FLinearColor> SuperSampledMaterial;
				TransferMaterial<PropertyType>(SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, InputMaterials, SuperSampledMaterial, UnresolvedColor);

				// Preserve the average luma when down sampling color.
				ProxyLOD::TGridWrapper<FLinearColor> SuperSampleGrid(SuperSampledMaterial, SuperSampledDstUVGrid.Size());
				SparseDownSampleColor(SuperSampleGrid, *LinearColorGrid);
			}
			else
			{

				TArray<FLinearColor> SuperSampledMaterial;
				TransferMaterial<PropertyType>(SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, InputMaterials, SuperSampledMaterial);

				// Generic down sample.
				ProxyLOD::TGridWrapper<FLinearColor> SuperSampleGrid(SuperSampledMaterial, SuperSampledDstUVGrid.Size());
				SparseDownSampleMaterial(SuperSampleGrid, *LinearColorGrid);

			}

			// Generate initial topology for dilation.  Cells with 'valid' data
			// will be marked with a '1' in the topology grid.

			ProxyLOD::TGrid<int32> TopologyGrid(TransferBufferSize.X, TransferBufferSize.Y);
			ConstrucTopologyStencilGrid(SuperSampledDstUVGrid, TopologyGrid);
			

			// Dilate the linear color buffer

			bool bDilationRequired = true;
			while (bDilationRequired)
			{
				// Each dilate propagates color into previsously invalid cells in the linear color grid
				// and updates the valid grid cell markers in the topology grid.
				bDilationRequired = DilateGrid(*LinearColorGrid, TopologyGrid);
			}

			// Pointer to a 2d grid that will be at the output resolution

			ProxyLOD::FLinearColorGrid::Ptr  SmallLinearColorGrid;
			
			// Down sample to the output resolution

			if (OutSize == LinearColorGrid->Size())
			{
				SmallLinearColorGrid = LinearColorGrid;
				
			}
			else
			{
				SmallLinearColorGrid = ProxyLOD::FLinearColorGrid::Create(OutSize.X, OutSize.Y);

				// Area weighted down sample of the grid.  Note that the texture data has been dilated from 
				// the UV islands already, so the downsample need not be sparse.

				DownSample2dGrid(*LinearColorGrid, *SmallLinearColorGrid);

				if (PropertyType == EFlattenMaterialProperties::Normal)
				{
					// The last step in down-sampling will not have produced normal vectors of unit legth.
					// renormalize the lenght.

					NormalizeLinearColorVectorGrid(*SmallLinearColorGrid);
					
				}
			}

			// Allocate the output

			TArray<FColor>& OutBuffer = OutMaterial.GetPropertySamples(PropertyType);
			ResizeArray(OutBuffer, OutSize.X * OutSize.Y);

			// Transfer the result into the FColor buffer needed for output.

			// NB: the normal map was just quantized.
			if (PropertyType == EFlattenMaterialProperties::Normal)
			{
				const FLinearColor* ColorData = SmallLinearColorGrid->GetData();

				ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, OutSize.X * OutSize.Y),
					[ColorData, &OutBuffer](const ProxyLOD::FIntRange& Range)
				{	
					FColor* OutColorData = OutBuffer.GetData();

					for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
					{
						OutColorData[i] = ColorData[i].QuantizeRound();
					}

				});
			}
			else
			{
				const FLinearColor* ColorData = SmallLinearColorGrid->GetData();
				ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, OutSize.X * OutSize.Y),
					[ColorData, &OutBuffer](const ProxyLOD::FIntRange& Range)
				{
					FColor* OutColorData = OutBuffer.GetData();

					for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
					{
						OutColorData[i] = ColorData[i].ToFColor(true);
					}

				});
			}

		}
	}

	// Map Diffuse, Specular, Metallic, Roughness, Normal, Emissive, Opacity to the correct materials.
	// NB: EFlattenMaterialProperties: SubSurface and AmbientOcclusion are not included
	void MapFlattenMaterial(const EFlattenMaterialProperties PropertyType,
		const FMeshDescription& DstRawMesh,
		const FMeshDescriptionArrayAdapter& SrcMeshAdapter,
		const ProxyLOD::FSrcDataGrid& SuperSampledCorrespondenceGrid,
		const ProxyLOD::FRasterGrid& SuperSampledDstUVGrid,
		const ProxyLOD::FRasterGrid& DstUVGrid,
		const TArray<FFlattenMaterial>& InputMaterials,
		const FColor UnresolvedSrcColor,
		FFlattenMaterial& OutMaterial)
	{
		switch (PropertyType)
		{
		case EFlattenMaterialProperties::Diffuse:
			TMapFlattenMaterial<EFlattenMaterialProperties::Diffuse>(DstRawMesh, SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, DstUVGrid, InputMaterials, OutMaterial, UnresolvedSrcColor);
			break;
		case EFlattenMaterialProperties::Specular:
			TMapFlattenMaterial<EFlattenMaterialProperties::Specular>(DstRawMesh, SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, DstUVGrid, InputMaterials, OutMaterial);
			break;
		case EFlattenMaterialProperties::Metallic:
			TMapFlattenMaterial<EFlattenMaterialProperties::Metallic>(DstRawMesh, SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, DstUVGrid, InputMaterials, OutMaterial);
			break;
		case EFlattenMaterialProperties::Roughness:
			TMapFlattenMaterial<EFlattenMaterialProperties::Roughness>(DstRawMesh, SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, DstUVGrid, InputMaterials, OutMaterial);
			break;
		case EFlattenMaterialProperties::Normal:
			TMapFlattenMaterial<EFlattenMaterialProperties::Normal>(DstRawMesh, SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, DstUVGrid, InputMaterials, OutMaterial);
			break;
		case EFlattenMaterialProperties::Emissive:
			TMapFlattenMaterial<EFlattenMaterialProperties::Emissive>(DstRawMesh, SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, DstUVGrid, InputMaterials, OutMaterial);
			break;
		case EFlattenMaterialProperties::Opacity:
			TMapFlattenMaterial<EFlattenMaterialProperties::Opacity>(DstRawMesh, SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, DstUVGrid, InputMaterials, OutMaterial);
			break;
		case EFlattenMaterialProperties::OpacityMask:
			TMapFlattenMaterial<EFlattenMaterialProperties::OpacityMask>(DstRawMesh, SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, DstUVGrid, InputMaterials, OutMaterial);
			break;
		default:
			return;
			// do nothing.  
		};
	}
}// end anonymous namespace

void ProxyLOD::MapFlattenMaterials(const FMeshDescription& DstRawMesh, const FMeshDescriptionArrayAdapter& SrcMeshAdapter,
	const ProxyLOD::FSrcDataGrid& SuperSampledCorrespondenceGrid,
	const ProxyLOD::FRasterGrid& SuperSampledDstUVGrid,
	const ProxyLOD::FRasterGrid& DstUVGrid,
	const TArray<FFlattenMaterial>& InputMaterials,
	const FColor UnresolvedSrcColor,
	FFlattenMaterial& OutMaterial)
{

	ProxyLOD::FTaskGroup TaskGroup;
	for (int32 i = 0; i < (int32)EFlattenMaterialProperties::NumFlattenMaterialProperties; ++i)
	{
		EFlattenMaterialProperties MaterialProperty = (EFlattenMaterialProperties)i;

		// NB: The MaterialProperty is captured by value

		auto MapTask = [MaterialProperty, &DstRawMesh, &SrcMeshAdapter, &SuperSampledCorrespondenceGrid, &SuperSampledDstUVGrid, &DstUVGrid, &InputMaterials, UnresolvedSrcColor, &OutMaterial]
		()
		{
			// This is also threaded internally. 

			MapFlattenMaterial(MaterialProperty, DstRawMesh, SrcMeshAdapter, SuperSampledCorrespondenceGrid, SuperSampledDstUVGrid, DstUVGrid, InputMaterials, UnresolvedSrcColor, OutMaterial);
		};

		// enqueue the task with the task manager.

		TaskGroup.Run(MapTask);
	}

	// This does the dispatch and waits to join

	TaskGroup.Wait();
}

// Maps each triangle id to a color.  This is just for testing
void ProxyLOD::ColorMapFlattedMaterials( const FVertexDataMesh& VertexDataMesh, 
	                                     const FMeshDescriptionArrayAdapter& MeshAdapter,
	                                     const ProxyLOD::FRasterGrid& UVGrid, 
	                                     const TArray<FFlattenMaterial>& InputMaterials, 
	                                     FFlattenMaterial& OutMaterial)
{
	// testing - coloring the simplified mesh by the partitions generated by uvatlas
	FColor Range[13] = { FColor(255, 0, 0), FColor(0, 255, 0), FColor(0, 0, 255), FColor(255, 255, 0), FColor(0, 255, 255),
		FColor(153, 102, 0), FColor(249, 129, 162), FColor(29, 143, 177), FColor(118, 42, 145),
		FColor(255, 121, 75), FColor(102, 204, 51), FColor(153, 153, 255), FColor(255, 255, 255) };


	TArray<FColor>& ColorBuffer = OutMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse);
	FIntPoint Size = OutMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse);
	ResizeArray(ColorBuffer, Size.X * Size.Y);


	for (int j = 0; j < Size.Y; ++j)
	{
		for (int i = 0; i < Size.X; ++i)
		{
			const auto& Data = UVGrid(i, j);
			if (Data.TriangleId > -1 || Data.SignedDistance < 0.)
			{
				uint32 TriangleId = Data.TriangleId;
				FLinearColor TriangleColor(Range[TriangleId % 13]);
				// intensity modulated by barycentric coords
				ProxyLOD::DArray3d BCoords = Data.BarycentricCoords;
				float Scale = (float)FMath::Min(FMath::Min(BCoords[0], BCoords[1]), BCoords[2]);
				TriangleColor *= Scale;
				ColorBuffer[i + j * Size.X] = TriangleColor.ToFColor(true);
			}
			else
			{
				ColorBuffer[i + j * Size.X] = FColor(0, 0, 0);
			}
		}
	}

	TArray<FColor> NormalMap = OutMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal);

	NormalMap = InputMaterials[0].GetPropertySamples(EFlattenMaterialProperties::Normal);
}

void ProxyLOD::CopyFlattenMaterial( const FFlattenMaterial& InMaterial, 
	                                FFlattenMaterial& OutMaterial)
{
	for (int32 i = 0; i < (int32)EFlattenMaterialProperties::NumFlattenMaterialProperties; ++i)
	{
		EFlattenMaterialProperties Property = (EFlattenMaterialProperties)i;
		if (InMaterial.DoesPropertyContainData(Property))
		{

			if (InMaterial.IsPropertyConstant(Property)) // copy into a full sized buffer.
			{
				TArray<FColor>& OutBuffer = OutMaterial.GetPropertySamples(Property);
				FIntPoint Size = OutMaterial.GetPropertySize(Property);
				ResizeArray(OutBuffer, Size.X * Size.Y);

				const TArray<FColor>& InBuffer = InMaterial.GetPropertySamples(Property);

				for (int32 j = 0; j < Size.X * Size.Y; j++)
				{
					OutBuffer[j] = InBuffer[0];
				}
			}
			else
			{
				TArray<FColor>& OutBuffer = OutMaterial.GetPropertySamples(Property);
				FIntPoint Size = OutMaterial.GetPropertySize(Property);
				ResizeArray(OutBuffer, Size.X * Size.Y);

				const TArray<FColor>& InBuffer = InMaterial.GetPropertySamples(Property);

				for (int32 j = 0; j < Size.X * Size.Y; j++)
				{
					OutBuffer[j] = InBuffer[j];
				}
			}
		}
	}
}