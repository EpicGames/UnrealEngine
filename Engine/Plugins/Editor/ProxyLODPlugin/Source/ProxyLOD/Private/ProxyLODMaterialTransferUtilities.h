// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialUtilities.h"

#include "ProxyLODMeshTypes.h"   // FMeshDescriptionArrayAdapter, FVertexDataMesh
#include "ProxyLODGrid2d.h"
#include "ProxyLODRasterizer.h" // FRasterGrid
#include "ProxyLODkDOPInterface.h"
#include "ProxyLODOpenVDB.h"

namespace ProxyLOD
{
	/*
	* Data type to be written in a 2d grid that records the locations on Source Geometry.
	*/
	class  FSrcMeshData;

	/**
	* Two dimensional grid implicitly mapping locations in texture atlas space to FSrcMeshData
	* that references locations on the source geometry and perhaps more information about how
	* that location was identified.
	*/
	typedef TGrid<FSrcMeshData>         FSrcDataGrid;

	/**
	* Generate a map between UV locations on the simplified geometry and points on the Src geometry.
	*
	* This is done by firing rays from locations on the reduced mesh polys that correspond to 
	* texel centers in a texture atlas grid space and recording the poly and uv-coordinate location 
	* hit in the SrcMesh along with additional information.
	* 
	* Texels in the resulting FSrcDataGrid that  have MaterialId = -1;
	*
	* @param SrcMesh             High poly grid that represents the source geometry.
	* @param SrckDOPTree         kDOPTree of the SrcMesh.
	* @param ReducedMesh         Low poly Mesh with UVs, that ostensibly represents the same geometry.
	* @param ReducedMeshUVGrid   A mapping of the UVs on the ReducedMesh to the texture atlas space.
	* @param TransferType        Controls the logic when a forward and reverse ray both hit high-res geometry.
	* @param MaxRayLength        The maximum distance a ray is allowed to travel.
	*
	* @return FSrcDataGrid  - A texel in the SrcDataGrid(i,j) holds hit information that corresponds to a ray fired from
	*                         the texel in ReducedMeshUVGrid(i,j).
	*
	* Because some locations in the texture atlas for the Reduced Mesh will fall outside of all the charts (i.e. be dead space),
	* there will be texels in the SrcDataGrid that correspond to this space and will be given .MaterialId =-1 for quick identification.
	*/
	FSrcDataGrid::Ptr CreateCorrespondence(
		const FMeshDescriptionArrayAdapter& SrcMesh,
		const FkDOPTree& SrckDOPTree,
		const FVertexDataMesh& ReducedMesh,
		const ProxyLOD::FRasterGrid& ReducedMeshUVGrid,
		const int32 TransferType, 
		float MaxRayLength = 3.f);
	/**
	* Generate a map between UV locations on the simplified geometry and points on the Src geometry.
	*
	* This is done by using an existing data structure that holds the Id of the closes Src geometry poly in a sparse voxel grid. 
	* The sparsity of the grid (i.e. only values fairly close the the boundary of the src geomgery are populated) limits the
	* utility of this functional variant.  In this approach the locations on the the reduced mesh polys that correspond to
	* texel centers in a texture atlas grid space are projected onto the closest Src geometry poly and a record the Src poly 
	* and uv-coordinate location within that poly are stored in the resulting grid.
	*
	* Texels in the resulting FSrcDataGrid that  have MaterialId = -1;
	*
	* @param ClosestSrcPolyGrid  Sparse 3d grid that holds the Id of the closest poly to a given voxel.
	* @param SrcMesh             High poly grid that represents the source geometry.
	* @param SrckDOPTree         kDOPTree of the SrcMesh.
	* @param ReducedMesh         Low poly Mesh with UVs, that ostensibly represents the same geometry.
	* @param ReducedMeshUVGrid   A mapping of the UVs on the ReducedMesh to the texture atlas space.
	* @param TransferType        Controls the logic when a forward and reverse ray both hit high-res geometry.
	* @param MaxRayLength        The maximum distance a ray is allowed to travel.
	*
	* @return FSrcDataGrid  - A texel in the SrcDataGrid(i,j) holds hit information that corresponds to a ray fired from
	*                         the texel in ReducedMeshUVGrid(i,j).
	*
	* Because some locations in the texture atlas for the Reduced Mesh will fall outside of all the charts (i.e. be dead space),
	* there will be texels in the SrcDataGrid that correspond to this space and will be given .MaterialId =-1 for quick identification.
	*/
	FSrcDataGrid::Ptr CreateCorrespondence(
		const openvdb::Int32Grid& ClosestSrcPolyGrid,
		const FMeshDescriptionArrayAdapter& SrcMesh,
		const FkDOPTree& SrckDOPTree,
		const FVertexDataMesh& ReducedMesh,
		const ProxyLOD::FRasterGrid& ReducedMeshUVGrid,
		const int32 TransferType, 
		float MaxRayLength = 3.f);

	/**
	* Utility used generate new Flattened Materials for the DstRawMesh that correspond to the super-sampling
	* of Flattened materials for the Source Geometry (SrcMeshAdapter).
	*
	* @param DstRawMesh                       Simplified Geometry
	* @param SrcMeshAdapter                   High poly source meshes
	* @param SuperSampledCorrespondenceGrid   High resolution Grid that maps locations in the texture atlas space
	*                                         ( of the Simplified Geometry ) to locations on the Src Geometry.
	* @param SuperSampledDstUVGrid            High resolution Grid that maps locations in the texture atlas space 
	*                                         ( of the Simplified Geometry) to positions on the Simplified Geometry.
	* @param InMaterials                      Array of flattened materials corresponding to the various meshes in the
	*                                         Src geometry.
	* @param UnresolvedSrcColor               Base color to be used when the src geometry can't be identified (e.g. regions where
	*                                         gap filling has closed doors and windows)
	* @param OutMaterials                     Flattened materials for the Simplified Geometry.
	*
	* Map Diffuse, Specular, Metallic, Roughness, Normal, Emissive, Opacity to the correct materials.
	* NB: EFlattenMaterialProperties: SubSurface, OpacityMask and AmbientOcclusion are not included
	*/
	void MapFlattenMaterials( const FMeshDescription& DstRawMesh,
		                      const FMeshDescriptionArrayAdapter&   SrcMeshAdapter,
							  const ProxyLOD::FSrcDataGrid& SuperSampledCorrespondenceGrid,
							  const ProxyLOD::FRasterGrid&  SuperSampledDstUVGrid,
							  const ProxyLOD::FRasterGrid&  DstUVGrid,
							  const TArray<FFlattenMaterial>& InMaterials, 
							  const FColor UnresolvedSrcColor,
		                      FFlattenMaterial& OutMaterial);


	/**
	* Testing Utility used generate new Flattened Materials for the DstRawMesh that corresponds to
	* mapping each triangle id to a color.
	*/
	void ColorMapFlattedMaterials( const FVertexDataMesh& VertexDataMesh, 
		                           const FMeshDescriptionArrayAdapter& MeshAdapter,
		                           const ProxyLOD::FRasterGrid& UVGrid, 
		                           const TArray<FFlattenMaterial>& InputMaterials, 
		                           FFlattenMaterial& OutMaterial);

	/**
	* Used for testing - copies a flattened material
	*/
	void CopyFlattenMaterial( const FFlattenMaterial& InMaterial, 
		                      FFlattenMaterial& OutMaterial);

	class FSrcMeshData
	{
	public:

		FSrcMeshData() { }

		FSrcMeshData(const FSrcMeshData& other) :
			TriangleId(other.TriangleId),
			MaterialId(other.MaterialId)
		{
			UV = other.UV;
			for (int32 i = 0; i < 3; ++i) BarycentricCoords[i] = other.BarycentricCoords[i];

			Forward = other.Forward;
			CoAligned = other.CoAligned;
			Handedness = other.Handedness;
		}

		void operator=(const FSrcMeshData& other)
		{
			TriangleId = other.TriangleId;
			MaterialId = other.MaterialId;
			UV = other.UV;

			for (int32 i = 0; i < 3; ++i) BarycentricCoords[i] = other.BarycentricCoords[i];

			Forward = other.Forward;
			CoAligned = other.CoAligned;
			Handedness = other.Handedness;
		}

		// @todo - why not int16?
		int32          TriangleId = -1;
		int32          MaterialId = -1;
		FVector2D      UV;
		DArray3d       BarycentricCoords = { 0., 0., 0. };

		// ray direction
		int32           Forward;

		// Both faces have normals that point (roughly) in the same direction?
		// if not, care should be taken in transferring the normal
		// particularly we see this when the original geometry was a shell with
		// double-sided material.

		int16           CoAligned = 1;

		// Record the UVs on the source poly have inverted handedness

		int16           Handedness = 1;  // either 1 or -1
	};
}