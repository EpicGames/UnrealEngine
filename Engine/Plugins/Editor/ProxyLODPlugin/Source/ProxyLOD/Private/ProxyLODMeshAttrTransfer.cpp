// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODMeshAttrTransfer.h"
#include "ProxyLODBarycentricUtilities.h" // for some of the barycentric stuff


template<int Size>
FColor AverageColor(const FColor(&Colors)[Size])
{
	// Accumulate with floats because FColor is only 8-bit
	float Tmp[4] = { 0,0,0,0 };
	for (int i = 0; i < Size; ++i)
	{
		Tmp[0] += Colors[i].R;
		Tmp[1] += Colors[i].G;
		Tmp[2] += Colors[i].B;
		Tmp[3] += Colors[i].A;
	}
	for (int i = 0; i < 4; ++i) Tmp[i] *= 1.f / float(Size);
	
	FColor Result;
	Result.R = IntCastChecked<uint8>( (int)(0.5f + Tmp[0] ) );
	Result.G = IntCastChecked<uint8>( (int)(0.5f + Tmp[1] ) );
	Result.B = IntCastChecked<uint8>( (int)(0.5f + Tmp[2] ) );
	Result.A = IntCastChecked<uint8>( (int)(0.5f + Tmp[3] ) );

	return Result;
}

template <int Size>
FVector3f AverageUnitVector(const FVector3f(&Vectors)[Size])
{
	FVector3f Result(0.f, 0.f, 0.f);

	for (int i = 0; i < Size; ++i)
	{
		Result += Vectors[i];
	}

	Result.Normalize();

	return Result;
}

template <int Size>
FVector2D AverageTexCoord(const FVector2D(&TexCoords)[Size])
{
	FVector2D Result(0.f, 0.f);

	for (int i = 0; i < Size; ++i)
	{
		Result += TexCoords[i];
	}

	Result *= 1.f / float(Size);

	return Result;
}

void ProxyLOD::TransferMeshAttributes(const FClosestPolyField& SrcPolyField, FMeshDescription& InOutMesh)
{
	const int32 NumFaces = InOutMesh.Polygons().Num();

	FStaticMeshAttributes Attributes(InOutMesh);

	TArrayView<const FVector3f> VertexPositions = Attributes.GetVertexPositions().GetRawArray();
	TArrayView<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals().GetRawArray();
	TArrayView<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents().GetRawArray();
	TArrayView<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns().GetRawArray();
	TArrayView<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors().GetRawArray();

	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	//const FMeshDescriptionArrayAdapter& RawMeshArrayAdapter = SrcPolyField.MeshAdapter();
	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumFaces),
		[&InOutMesh, &SrcPolyField, &VertexPositions, &VertexInstanceTangents, &VertexInstanceBinormalSigns, &VertexInstanceNormals, &VertexInstanceUVs, &VertexInstanceColors, &PolygonGroupImportedMaterialSlotNames](const ProxyLOD::FIntRange& Range)
	{
		FClosestPolyField::FPolyConstAccessor ConstPolyAccessor = SrcPolyField.GetPolyConstAccessor();

		// loop over the faces
		for (int32 CurrentRange = Range.begin(), EndRange = Range.end(); CurrentRange < EndRange; ++CurrentRange)
		{
			FPolygonID PolygonID = FPolygonID(CurrentRange);
			int32 LastMaterialIndex = -1;
			for (const FTriangleID TriangleID : InOutMesh.GetPolygonTriangles(PolygonID))
			{
				// get the three corners for this Triangle
				TArrayView<const FVertexInstanceID> TriangleVerts = InOutMesh.GetTriangleVertexInstances(TriangleID);

				for (int32 i = 0; i < 3; ++i)
				{
					const FVertexInstanceID Idx = TriangleVerts[i];
					// world space location
					const FVector3f& WSPos = VertexPositions[InOutMesh.GetVertexInstanceVertex(Idx)];

					bool bFoundPoly;
					// The closest poly to this point
					FMeshDescriptionArrayAdapter::FRawPoly RawPoly = ConstPolyAccessor.Get(WSPos, bFoundPoly);

					LastMaterialIndex = RawPoly.FaceMaterialIndex;

					// ----  Transfer the face - average values to each wedge --- //
					// NB: might replace with something more sophisticated later.

					// Compute the average color
					VertexInstanceColors[Idx] = FVector4f(FLinearColor(AverageColor(RawPoly.WedgeColors)));

					// The average Tangent Vectors
					VertexInstanceTangents[Idx] = AverageUnitVector(RawPoly.WedgeTangentX);
					VertexInstanceNormals[Idx] = AverageUnitVector(RawPoly.WedgeTangentZ);
					VertexInstanceBinormalSigns[Idx] = GetBasisDeterminantSign((FVector)VertexInstanceTangents[Idx], (FVector)AverageUnitVector(RawPoly.WedgeTangentY), (FVector)VertexInstanceNormals[Idx]);

					// Average Texture Coords
					VertexInstanceUVs.Set(Idx, 0, FVector2f(AverageTexCoord(RawPoly.WedgeTexCoords[0])));	// LWC_TODO: Precision loss
				}
			}
			// Assign the material index that the last vertex of this face sees.
			if (LastMaterialIndex != -1)
			{
				FPolygonGroupID PolygonGroupID(LastMaterialIndex);
				if (!InOutMesh.IsPolygonGroupValid(PolygonGroupID))
				{
					// @todo: check: this doesn't seem thread-safe to me - RichardTW
					InOutMesh.CreatePolygonGroupWithID(PolygonGroupID);
					PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = FName(*FString::Printf(TEXT("ProxyLOD_Material_%d"), FMath::Rand()));
				}
				InOutMesh.SetPolygonPolygonGroup(PolygonID, PolygonGroupID);
			}
		}
	}
	);
}


// Transfers the normals from the source geometry to the ArrayOfStructs mesh.
void ProxyLOD::TransferSrcNormals(const FClosestPolyField& SrcPolyField, FAOSMesh& InOutMesh)
{

	const uint32 NumVertexes = InOutMesh.GetNumVertexes();
	FPositionNormalVertex* Vertexes = InOutMesh.Vertexes;

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumVertexes),
		[&Vertexes, &SrcPolyField](const ProxyLOD::FUIntRange& Range)
	{
		const auto PolyAccessor = SrcPolyField.GetPolyConstAccessor();
		for (uint32 CurrentRange = Range.begin(), EndRange = Range.end(); CurrentRange < EndRange; ++CurrentRange)
		{
			// Get the closest poly to this vertex.
			FVector3f& Normal = Vertexes[CurrentRange].Normal;
			const FVector3f& Pos = Vertexes[CurrentRange].Position;

			bool bSuccess = false;
			const FMeshDescriptionArrayAdapter::FRawPoly  RawPoly = PolyAccessor.Get(openvdb::Vec3d(Pos.X, Pos.Y, Pos.Z), bSuccess);
			if (bSuccess)
			{
				bool bValidSizedPoly = true;
#if 0
				// compute 4 * area * area  of raw poly
				const FVector3f AB = RawPoly.VertexPositions[1] - RawPoly.VertexPositions[0];
				const FVector3f AC = RawPoly.VertexPositions[2] - RawPoly.VertexPositions[0];
				const float FourAreaSqr = FVector3f::CrossProduct(AB, AC).SizeSquared();

				bValidSizedPoly = FourAreaSqr > 0.001;
#endif
				// Compute the barycentric weights of the vertex projected onto the nearest face.


				const auto Weights = ProxyLOD::ComputeBarycentricWeights(RawPoly.VertexPositions, Pos);

				bool MissedPoly = Weights[0] > 1 || Weights[0] < 0 || Weights[1] > 1 || Weights[1] < 0 || Weights[2] > 1 || Weights[2] < 0;

				if (!MissedPoly && bValidSizedPoly)
				{
					FVector3f TransferedNormal = ProxyLOD::InterpolateVertexData(Weights, RawPoly.WedgeTangentZ);
					//bool bNormal = !TransferedNormal.ContainsNaN();
					bool bNormal = TransferedNormal.Normalize(0.1f);
					// assume that the transfered normal is more accurate if it is somewhat aligned with the local geometric normal
					if (bNormal && FVector3f::DotProduct(TransferedNormal, Normal) > 0.2f)
					{
						Normal += 3.f * TransferedNormal;
						Normal.Normalize();
					}

				}
			}

		}
	});

};


void ProxyLOD::TransferVertexColors(const FClosestPolyField& SrcPolyField, FMeshDescription& InOutMesh)
{
	TArrayView<const FVector3f> VertexPositions = InOutMesh.GetVertexPositions().GetRawArray();

	FStaticMeshAttributes Attributes(InOutMesh);
	TArrayView<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors().GetRawArray();

	const uint32 NumWedges = InOutMesh.VertexInstances().Num();


	// Loop over the polys in the result mesh.

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumWedges),
		[&VertexPositions, &VertexInstanceColors, &InOutMesh, &SrcPolyField](const ProxyLOD::FUIntRange& Range)
	{
		const auto PolyAccessor = SrcPolyField.GetPolyConstAccessor();

		// loop over wedges.
		for (uint32 CurrentRange = Range.begin(), EndRange = Range.end(); CurrentRange < EndRange; ++CurrentRange)
		{
			// Get the closest poly to this vertex.
			FVertexInstanceID VertexInstanceID(CurrentRange);
			const FVector& Pos = (FVector)VertexPositions[InOutMesh.GetVertexInstanceVertex(VertexInstanceID)];

			// Find the closest poly to this wedge.  
			// NB: all wedges that share a vert location will end up with the same color this way

			bool bSuccess = false;
			const FMeshDescriptionArrayAdapter::FRawPoly  RawPoly = PolyAccessor.Get(openvdb::Vec3d(Pos.X, Pos.Y, Pos.Z), bSuccess);
			
			// default to white
			VertexInstanceColors[VertexInstanceID] = FVector4f(FLinearColor::White);
			if (bSuccess)
			{

				// Compute the barycentric weights of the vertex projected onto the nearest face.
				// We use these to determine the closest corner of the poly

				ProxyLOD::DArray3d Weights = ProxyLOD::ComputeBarycentricWeights(RawPoly.VertexPositions, (FVector3f)Pos);

				bool MissedPoly = Weights[0] > 1 || Weights[0] < 0 || Weights[1] > 1 || Weights[1] < 0 || Weights[2] > 1 || Weights[2] < 0;

				if (!MissedPoly)
				{
					FLinearColor WedgeColors[3] = { FLinearColor(RawPoly.WedgeColors[0]), 
						                            FLinearColor(RawPoly.WedgeColors[1]), 
						                            FLinearColor(RawPoly.WedgeColors[2]) };

					FLinearColor InterpolatedColor = InterpolateVertexData(Weights, WedgeColors);

					float AveLum = WedgeColors[0].GetLuminance() + WedgeColors[1].GetLuminance() + WedgeColors[2].GetLuminance();
					AveLum /= 3.f;

					float LumTmp = InterpolatedColor.GetLuminance();
					if (LumTmp > 1.e-5 && AveLum > 1.e-5)
					{
						InterpolatedColor *= AveLum / LumTmp;
					}

					// fix up the intensity.

					VertexInstanceColors[VertexInstanceID] = FVector4f(InterpolatedColor);
				}
			}
			
		}
	});
}


template <typename ProjectionOperatorType>
void ProjectVerticiesOntoSrc(const ProjectionOperatorType& ProjectionOperator, const FClosestPolyField& SrcPolyField, FMeshDescription& InOutMesh)
{
	TArrayView<FVector3f> VertexPositions = InOutMesh.GetVertexPositions().GetRawArray();
	const uint32 NumVertexes = InOutMesh.Vertices().Num();

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumVertexes),
		[VertexPositions, &SrcPolyField, ProjectionOperator](const ProxyLOD::FUIntRange& Range)
	{
		const auto PolyAccessor = SrcPolyField.GetPolyConstAccessor();

		for (uint32 CurrentRange = Range.begin(), EndRange = Range.end(); CurrentRange < EndRange; ++CurrentRange)
		{
			FVertexID VertexID(CurrentRange);
			// Get the closest poly to this vertex.
			FVector3f& Pos = VertexPositions[VertexID];

			bool bSuccess = false;
			const FMeshDescriptionArrayAdapter::FRawPoly  RawPoly = PolyAccessor.Get(openvdb::Vec3d(Pos.X, Pos.Y, Pos.Z), bSuccess);
			if (bSuccess)
			{

				// Compute the barycentric weights of the vertex projected onto the nearest face.
				// We use these to determine the closest corner of the poly

				ProxyLOD::DArray3d Weights = ProxyLOD::ComputeBarycentricWeights(RawPoly.VertexPositions, Pos);

				bool MissedPoly = Weights[0] > 1 || Weights[0] < 0 || Weights[1] > 1 || Weights[1] < 0 || Weights[2] > 1 || Weights[2] < 0;

				if (!MissedPoly)

				{
					Pos = (FVector3f)ProjectionOperator(Weights, RawPoly.VertexPositions, (FVector)Pos);
				}
			}
		}
	});
}

template <typename ProjectionOperatorType>
void ProjectVerticiesOntoSrc(const ProjectionOperatorType& ProjectionOperator, const FClosestPolyField& SrcPolyField, FVertexDataMesh& InOutMesh)
{
	const uint32 NumVertexes = InOutMesh.Points.Num();
	FVector3f* Vertexes = InOutMesh.Points.GetData();

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumVertexes),
		[Vertexes, &SrcPolyField, ProjectionOperator](const ProxyLOD::FUIntRange& Range)
	{
		const auto PolyAccessor = SrcPolyField.GetPolyConstAccessor();

		for (uint32 CurrentRange = Range.begin(), EndRange = Range.end(); CurrentRange < EndRange; ++CurrentRange)
		{
			// Get the closest poly to this vertex.
			FVector3f& Pos = Vertexes[CurrentRange];

			bool bSuccess = false;
			const FMeshDescriptionArrayAdapter::FRawPoly  RawPoly = PolyAccessor.Get(openvdb::Vec3d(Pos.X, Pos.Y, Pos.Z), bSuccess);
			if (bSuccess)
			{

				// Compute the barycentric weights of the vertex projected onto the nearest face.
				// We use these to determine the closest corner of the poly

				ProxyLOD::DArray3d Weights = ProxyLOD::ComputeBarycentricWeights(RawPoly.VertexPositions, Pos);

				bool MissedPoly = Weights[0] > 1 || Weights[0] < 0 || Weights[1] > 1 || Weights[1] < 0 || Weights[2] > 1 || Weights[2] < 0;

				if (!MissedPoly)

				{
					Pos = (FVector3f)ProjectionOperator(Weights, RawPoly.VertexPositions, (FVector)Pos);
				}
			}
		}
	});
}

template <typename ProjectionOperatorType, typename T>
void ProjectVerticiesOntoSrc(const ProjectionOperatorType& ProjectionOperator, const FClosestPolyField& SrcPolyField, TAOSMesh<T>& InOutMesh)
{
	uint32 NumVertexes = InOutMesh.GetNumVertexes();
	T* Vertexes = InOutMesh.Vertexes;

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumVertexes),
		[Vertexes, &SrcPolyField, ProjectionOperator](const ProxyLOD::FUIntRange& Range)
	{
		const auto PolyAccessor = SrcPolyField.GetPolyConstAccessor();

		for (uint32 CurrentRange = Range.begin(), EndRange = Range.end(); CurrentRange < EndRange; ++CurrentRange)
		{
			// Get the closest poly to this vertex.
			FVector3f& Pos = Vertexes[CurrentRange].Position;

			bool bSuccess = false;
			const FMeshDescriptionArrayAdapter::FRawPoly  RawPoly = PolyAccessor.Get(openvdb::Vec3d(Pos.X, Pos.Y, Pos.Z), bSuccess);
			if (bSuccess)
			{

				// Compute the barycentric weights of the vertex projected onto the nearest face.
				// We use these to determine the closest corner of the poly

				ProxyLOD::DArray3d Weights = ProxyLOD::ComputeBarycentricWeights(RawPoly.VertexPositions, Pos);

				bool MissedPoly = Weights[0] > 1 || Weights[0] < 0 || Weights[1] > 1 || Weights[1] < 0 || Weights[2] > 1 || Weights[2] < 0;

				if (!MissedPoly)

				{
					Pos = (FVector3f)ProjectionOperator(Weights, RawPoly.VertexPositions, (FVector)Pos);
				}
			}
		}
	});
}

class FSnapProjectionOperator
{
public:
	FSnapProjectionOperator(float MaxDistSqr) 
		: MaxCloseDistSqr(MaxDistSqr) 
	{}

	FVector operator()(const ProxyLOD::DArray3d& Weights, const FVector3f(&VertexPos)[3], const FVector& CurrentPos) const 
	{
		// Identify the closest vertex.

		int MinIdx = 0;
		if (Weights[1] > Weights[MinIdx]) MinIdx = 1;
		if (Weights[2] > Weights[MinIdx]) MinIdx = 2;

		// Form a vector to the closest vertex.

		FVector ToClosestVertex = (FVector)VertexPos[MinIdx] - CurrentPos;

		// Test distance to the closest vertex, if we are further than the cutoff, we
		// just project the vertex onto the surface.
		FVector ResultPos = 0.1 * (FVector)CurrentPos;
		if (ToClosestVertex.SizeSquared() < MaxCloseDistSqr)
		{
			ResultPos += 0.9f * (FVector)VertexPos[MinIdx];
		}
		else // just project
		{
			ResultPos += 0.9f * (FVector)ProxyLOD::InterpolateVertexData(Weights, VertexPos);
		}

		return ResultPos;
	}
private:
	float MaxCloseDistSqr;

};

void ProxyLOD::ProjectVertexWithSnapToNearest(const FClosestPolyField& SrcPolyField, FMeshDescription& InOutMesh)
{

	const double VoxelSize = SrcPolyField.GetVoxelSize();

	// 3 voxel distance.  When projecting to the nearest vert, use this as a distance cut off.
	float MaxCloseDistSqr = 9.f * VoxelSize * VoxelSize;

	// Do the work

	ProjectVerticiesOntoSrc(FSnapProjectionOperator(MaxCloseDistSqr), SrcPolyField, InOutMesh);

}


void ProxyLOD::ProjectVertexWithSnapToNearest(const FClosestPolyField& SrcPolyField, FAOSMesh& InOutMesh)
{

	const double VoxelSize = SrcPolyField.GetVoxelSize();

	// 3 voxel distance.  When projecting to the nearest vert, use this as a distance cut off.
	float MaxCloseDistSqr = 9.f * VoxelSize * VoxelSize;

	// Do the work

	ProjectVerticiesOntoSrc(FSnapProjectionOperator(MaxCloseDistSqr), SrcPolyField, InOutMesh);

}


void ProxyLOD::ProjectVertexWithSnapToNearest(const FClosestPolyField& SrcPolyField, FVertexDataMesh& InOutMesh)
{

	const double VoxelSize = SrcPolyField.GetVoxelSize();

	// 3 voxel distance.  When projecting to the nearest vert, use this as a distance cut off.
	float MaxCloseDistSqr = 9.f * VoxelSize * VoxelSize;

	// Do the work

	ProjectVerticiesOntoSrc(FSnapProjectionOperator(MaxCloseDistSqr), SrcPolyField, InOutMesh);

}

class FProjectionOperator
{
public:
	FProjectionOperator(float MaxDistSqr)
		: MaxCloseDistSqr(MaxDistSqr)
	{}

	FVector operator()(const ProxyLOD::DArray3d& Weights, const FVector3f(&VertexPos)[3], const FVector& CurrentPos) const
	{

		// Closest location on the surface

		const FVector ProjectedLocation = (FVector)ProxyLOD::InterpolateVertexData(Weights, VertexPos);

		// Form a vector to the closest vertex.

		FVector ToClosestVertex = ProjectedLocation - CurrentPos;

		// Test distance to the closest vertex, if we are further than the cutoff, we
		// just project the vertex onto the surface.
		FVector ResultPos = CurrentPos;
		if (ToClosestVertex.SizeSquared() < MaxCloseDistSqr)
		{
			ResultPos = 0.25f * CurrentPos +  0.75f * ProjectedLocation;
		}

		return ResultPos;
	}
private:
	float MaxCloseDistSqr;

};

void ProxyLOD::ProjectVertexOntoSrcSurface(const FClosestPolyField& SrcPolyField, FMeshDescription& InOutMesh)
{

	const double VoxelSize = SrcPolyField.GetVoxelSize();

	// 3 voxel distance.  When projecting to the nearest vert, use this as a distance cut off.
	float MaxCloseDistSqr = 9.f * VoxelSize * VoxelSize;

	// Do the work

	ProjectVerticiesOntoSrc(FProjectionOperator(MaxCloseDistSqr), SrcPolyField, InOutMesh);

}

void ProxyLOD::ProjectVertexOntoSrcSurface(const FClosestPolyField& SrcPolyField, FAOSMesh& InOutMesh)
{

	const double VoxelSize = SrcPolyField.GetVoxelSize();

	// 3 voxel distance.  When projecting to the nearest vert, use this as a distance cut off.
	float MaxCloseDistSqr = 9.f * VoxelSize * VoxelSize;

	// Do the work

	ProjectVerticiesOntoSrc(FProjectionOperator(MaxCloseDistSqr), SrcPolyField, InOutMesh);

}

void ProxyLOD::ProjectVertexOntoSrcSurface(const FClosestPolyField& SrcPolyField, FVertexDataMesh& InOutMesh)
{

	const double VoxelSize = SrcPolyField.GetVoxelSize();

	// 3 voxel distance.  When projecting to the nearest vert, use this as a distance cut off.
	float MaxCloseDistSqr = 9.f * VoxelSize * VoxelSize;

	// Do the work

	ProjectVerticiesOntoSrc(FProjectionOperator(MaxCloseDistSqr), SrcPolyField, InOutMesh);

}

