// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProxyLODMeshTypes.h"
#include "ProxyLODThreadedWrappers.h"
#include "ProxyLODOpenVDB.h"

namespace ProxyLOD
{
	/** 
	* Transfers the Barycenteric-averaged wedge colors from a high poly mesh to a simplified version  
	* of the same geometry.  
	*
	* NB: This is not a general purpose tool as it assumes the simplified geometry is within a small fixed distance
	*     of the source geometry.    
	*
	* @param  SrcPolyField  The closest poly field derived from the high poly src mesh
	* @param  InOutMesh     The target raw mesh.  Only the wedge colors on this target are updated.
	*/
	void TransferVertexColors(const FClosestPolyField& SrcPolyField, FMeshDescription& InOutMesh);

	
	/**
	* Transfers Barycenteric-averaged vertex normals from a high poly mesh src mesh to a simplified version
	* of the same geometry.
	*
	* NB: This is not a general purpose tool as it assumes the simplified geometry is within a small fixed distance
	*     of the source geometry.
	*
	* @param  SrcPolyField  The closest poly field derived from the high poly src mesh
	* @param  InOutMesh     The target Array Of Structs Mesh.  Only the normal is updated.
	*/
	void TransferSrcNormals( const FClosestPolyField& SrcPolyField, FAOSMesh& InOutMesh );

	/**
	* No OP! 
	*
	* NB: This function is a No Op since the target mesh type does not support normals.  
	*     The function only exists to simplify some templated code.
	*
	* @param  SrcPolyField  The closest poly field derived from the high poly src mesh
	* @param  InOutMesh     The target Position Only mesh Mesh.  Only the normal is updated.
	*/
	static void TransferSrcNormals( const FClosestPolyField& SrcPolyField, FPosSimplifierMesh& InOutMesh) {};


	/**
	* Update the vertex position of low poly mesh to better align with a high poly src geometry.
	* This will attempt to snap a low poly vertex to the location of the closest high poly source vertex
	* if one is close, otherwise it will simply project the vertex 90% of the distance to the source surface.
	* All other vertex attributes on the InOutMesh (e.g. Tangent Space) are unchanged by this.
	*
	* NB: This is not a general purpose tool as it assumes the simplified geometry is within a small fixed distance
	*     of the source geometry.
	
	*
	* @param  SrcPolyField  The closest poly field derived from the high poly src mesh
	* @param  InOutMesh     The target low poly mesh.  Only the position is updated.
	*/
	void ProjectVertexWithSnapToNearest( const FClosestPolyField& SrcPolyField, FMeshDescription& InOutMesh );
	void ProjectVertexWithSnapToNearest( const FClosestPolyField& SrcPolyField, FAOSMesh& InOutMesh );
	void ProjectVertexWithSnapToNearest( const FClosestPolyField& SrcPolyField, FVertexDataMesh& InOutMesh);

	/**
	* Update the vertex position of low poly mesh to better align with a high poly src geometry.
	* This will simply project the vertex 90% of the distance to the source surface.
	* All other vertex attributes on the InOutMesh (e.g. Tangent Space) are unchanged by this.
	*
	* NB: This is not a general purpose tool as it assumes the simplified geometry is within a small fixed distance
	*     of the source geometry.

	*
	* @param  SrcPolyField  The closest poly field derived from the high poly src mesh
	* @param  InOutMesh     The target low poly mesh.  Only the position is updated.
	*/
	void ProjectVertexOntoSrcSurface( const FClosestPolyField& SrcPolyField, FMeshDescription& InOutMesh );
	void ProjectVertexOntoSrcSurface( const FClosestPolyField& SrcPolyField, FAOSMesh& InOutMesh );
	void ProjectVertexOntoSrcSurface( const FClosestPolyField& SrcPolyField, FVertexDataMesh& InOutMesh );



	/**
	* Primarily intended as testing code. Update the tangent space, wedge colors, and first texture coords with the face-averaged
	* value of the closest poly in the high poly SrcPolyField.
	*
	* NB: This is not a general purpose tool as it assumes the simplified geometry is within a small fixed distance
	*     of the source geometry.

	*
	* @param  SrcPolyField  The closest poly field derived from the high poly src mesh
	* @param  InOutMesh     The target low poly mesh.
	*/
	void TransferMeshAttributes(const FClosestPolyField& SrcPolyField, FMeshDescription& InOutMesh);

}

