// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODMeshConvertUtils.h" 
#include "ProxyLODMeshUtilities.h"

#include "StaticMeshOperations.h"


static void ResetStaticMeshDescription(FMeshDescription& MeshDecription)
{
	MeshDecription.Empty();
	FStaticMeshAttributes Attributes(MeshDecription);
	Attributes.Register();
}

// Convert QuadMesh to Triangles by splitting
void ProxyLOD::MixedPolyMeshToRawMesh(const FMixedPolyMesh& SimpleMesh, FMeshDescription& DstRawMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::MixedPolyMeshToRawMesh)

	ResetStaticMeshDescription(DstRawMesh);

	FStaticMeshAttributes Attributes(DstRawMesh);
	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	// Splitting a quad doesn't introduce any new verts.
	const uint32 DstNumVerts = SimpleMesh.Points.size();

	const uint32 NumQuads = SimpleMesh.Quads.size();

	// Each quad becomes 2 triangles.
	const uint32 DstNumTris = 2 * NumQuads + SimpleMesh.Triangles.size();

	// Each Triangle has 3 corners
	const uint32 DstNumIndexes = 3 * DstNumTris;

	if (VertexInstanceUVs.GetNumChannels() < 1)
	{
		VertexInstanceUVs.SetNumChannels(1);
	}

	FPolygonGroupID PolygonGroupID = INDEX_NONE;
	if (DstRawMesh.PolygonGroups().Num() == 0)
	{
		PolygonGroupID = DstRawMesh.CreatePolygonGroup();
		PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = FName(*FString::Printf(TEXT("ProxyLOD_Material_%d"), FMath::Rand()));
	}
	else
	{
		PolygonGroupID = DstRawMesh.PolygonGroups().GetFirstValidID();
	}

	// Copy the vertices over
	TMap<int32, FVertexID> VertexIDMap;
	VertexIDMap.Reserve(DstNumVerts);
	{
		for (uint32 i = 0, I = DstNumVerts; i < I; ++i)
		{
			const openvdb::Vec3s& Vertex = SimpleMesh.Points[i];
			const FVertexID NewVertexID = DstRawMesh.CreateVertex();
			VertexPositions[NewVertexID] = FVector3f(Vertex[0], Vertex[1], Vertex[2]);
			VertexIDMap.Add(i, NewVertexID);
		}
	}

	// Connectivity:
	auto CreateTriangle = [&DstRawMesh, PolygonGroupID, &VertexInstanceNormals, &VertexInstanceTangents, &VertexInstanceBinormalSigns, &VertexInstanceColors, &VertexInstanceUVs, &EdgeHardnesses](FVertexID TriangleIndex[3])
	{
		TArray<FVertexInstanceID> VertexInstanceIDs;
		VertexInstanceIDs.SetNum(3);
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			VertexInstanceIDs[Corner] = DstRawMesh.CreateVertexInstance(TriangleIndex[Corner]);
			VertexInstanceTangents[VertexInstanceIDs[Corner]] = FVector3f(1, 0, 0);
			VertexInstanceNormals[VertexInstanceIDs[Corner]] = FVector3f(0, 0, 1);
			VertexInstanceBinormalSigns[VertexInstanceIDs[Corner]] = GetBasisDeterminantSign((FVector)VertexInstanceTangents[VertexInstanceIDs[Corner]].GetSafeNormal(),
																							 (FVector)(VertexInstanceNormals[VertexInstanceIDs[Corner]] ^ VertexInstanceTangents[VertexInstanceIDs[Corner]]).GetSafeNormal(),
																							 (FVector)VertexInstanceNormals[VertexInstanceIDs[Corner]].GetSafeNormal());
			VertexInstanceColors[VertexInstanceIDs[Corner]] = FVector4f(0.0f);
			VertexInstanceUVs.Set(VertexInstanceIDs[Corner], 0, FVector2f(0.0f, 0.0f));
		}

		// Insert a polygon into the mesh
		DstRawMesh.CreatePolygon(PolygonGroupID, VertexInstanceIDs);
	};

	{
		for (uint32 q = 0, Q = NumQuads; q < Q; ++q)
		{
			const openvdb::Vec4I& Quad = SimpleMesh.Quads[q];
			// add as two triangles to raw mesh
			bool bClockWiseTriangle = true;
#if (PROXYLOD_CLOCKWISE_TRIANGLES != 1)
			bClockWiseTriangle = false;
#endif
			FVertexID VertexIndexes[3];
			VertexIndexes[0] = VertexIDMap[Quad[0]];
			VertexIndexes[1] = bClockWiseTriangle ? VertexIDMap[Quad[1]] : VertexIDMap[Quad[3]];
			VertexIndexes[2] = VertexIDMap[Quad[2]];
			CreateTriangle(VertexIndexes);
			VertexIndexes[0] = VertexIDMap[Quad[2]];
			VertexIndexes[1] = bClockWiseTriangle ? VertexIDMap[Quad[3]] : VertexIDMap[Quad[1]];
			VertexIndexes[2] = VertexIDMap[Quad[0]];
			CreateTriangle(VertexIndexes);
		}

		// add the SimpleMesh triangles.
		uint32 IndexStop = SimpleMesh.Triangles.size();
		for (uint32 t = 0, T = IndexStop; t < T; ++t)
		{
			const openvdb::Vec3I& Tri = SimpleMesh.Triangles[t];
			bool bClockWiseTriangle = true;
#if (PROXYLOD_CLOCKWISE_TRIANGLES != 1)
			bClockWiseTriangle = false;
#endif
			FVertexID VertexIndexes[3];
			VertexIndexes[0] = bClockWiseTriangle ? VertexIDMap[Tri[0]] : VertexIDMap[Tri[2]];
			VertexIndexes[1] = VertexIDMap[Tri[1]];
			VertexIndexes[2] = bClockWiseTriangle ? VertexIDMap[Tri[2]] : VertexIDMap[Tri[0]];
			CreateTriangle(VertexIndexes);
		}
	}
}


void ProxyLOD::AOSMeshToRawMesh(const FAOSMesh& AOSMesh, FMeshDescription& OutRawMesh)
{

	ResetStaticMeshDescription(OutRawMesh);

	FStaticMeshAttributes Attributes(OutRawMesh);
	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	const uint32 DstNumPositions = AOSMesh.GetNumVertexes();
	const uint32 DstNumIndexes = AOSMesh.GetNumIndexes();

	if (VertexInstanceUVs.GetNumChannels() < 1)
	{
		VertexInstanceUVs.SetNumChannels(1);
	}

	FPolygonGroupID PolygonGroupID = INDEX_NONE;
	if (OutRawMesh.PolygonGroups().Num() == 0)
	{
		PolygonGroupID = OutRawMesh.CreatePolygonGroup();
		PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = FName(*FString::Printf(TEXT("ProxyLOD_Material_%d"), FMath::Rand()));
	}
	else
	{
		PolygonGroupID = OutRawMesh.PolygonGroups().GetFirstValidID();
	}

	checkSlow(DstNumIndexes % 3 == 0);
	// Copy the vertices over
	TMap<int32, FVertexID> VertexIDMap;
	VertexIDMap.Reserve(DstNumPositions);
	{
		const auto& AOSVertexes = AOSMesh.Vertexes;
		for (uint32 i = 0, I = DstNumPositions; i < I; ++i)
		{
			const FVector3f& Position = AOSVertexes[i].GetPos();
			const FVertexID NewVertexID = OutRawMesh.CreateVertex();
			VertexPositions[NewVertexID] = Position;
			VertexIDMap.Add(i, NewVertexID);
		}

		checkSlow(VertexPositions.GetNumElements() == DstNumPositions);
	}

	const uint32* AOSIndexes = AOSMesh.Indexes;

	// Connectivity: 
	auto CreateTriangle = [&OutRawMesh, PolygonGroupID, &VertexInstanceNormals, &VertexInstanceTangents, &VertexInstanceBinormalSigns, &VertexInstanceColors, &VertexInstanceUVs, &EdgeHardnesses](const FVertexID TriangleIndex[3], const FVector3f Normals[3])
	{
		TArray<FVertexInstanceID> VertexInstanceIDs;
		VertexInstanceIDs.SetNum(3);
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			VertexInstanceIDs[Corner] = OutRawMesh.CreateVertexInstance(TriangleIndex[Corner]);
			VertexInstanceTangents[VertexInstanceIDs[Corner]] = FVector3f(1, 0, 0);
			VertexInstanceNormals[VertexInstanceIDs[Corner]] = Normals[Corner];
			VertexInstanceBinormalSigns[VertexInstanceIDs[Corner]] = GetBasisDeterminantSign((FVector)VertexInstanceTangents[VertexInstanceIDs[Corner]].GetSafeNormal(),
																							 (FVector)(VertexInstanceNormals[VertexInstanceIDs[Corner]] ^ VertexInstanceTangents[VertexInstanceIDs[Corner]]).GetSafeNormal(),
																							 (FVector)VertexInstanceNormals[VertexInstanceIDs[Corner]].GetSafeNormal());
			VertexInstanceColors[VertexInstanceIDs[Corner]] = FVector4f(1.0f);
			VertexInstanceUVs.Set(VertexInstanceIDs[Corner], 0, FVector2f(0.0f, 0.0f));
		}

		// Insert a polygon into the mesh
		OutRawMesh.CreatePolygon(PolygonGroupID, VertexInstanceIDs);
	};

	{
		uint32 IndexStop = DstNumIndexes/3;
		for (uint32 t = 0, T = IndexStop; t < T; ++t)
		{
			FVertexID VertexIndexes[3];
			FVector3f Normals[3];
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				VertexIndexes[Corner] = VertexIDMap[AOSIndexes[(t*3) + Corner]];
				const auto& AOSVertex = AOSMesh.Vertexes[AOSIndexes[(t*3) + Corner]];
				Normals[Corner] = AOSVertex.Normal;
			}
			CreateTriangle(VertexIndexes, Normals);
		}
	}
}


void ProxyLOD::VertexDataMeshToRawMesh(const FVertexDataMesh& SrcVertexDataMesh, FMeshDescription& OutRawMesh)
{
	ResetStaticMeshDescription(OutRawMesh);

	FStaticMeshAttributes Attributes(OutRawMesh);
	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	const uint32 DstNumPositions = SrcVertexDataMesh.Points.Num();
	const uint32 DstNumIndexes = SrcVertexDataMesh.Indices.Num();
	const uint32 SrcNumTriangles = DstNumIndexes / 3;
	
	FPolygonGroupID PolygonGroupID = INDEX_NONE;
	if (OutRawMesh.PolygonGroups().Num() == 0)
	{
		PolygonGroupID = OutRawMesh.CreatePolygonGroup();
		PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = FName(*FString::Printf(TEXT("ProxyLOD_Material_%d"), FMath::Rand()));
	}
	else
	{
		PolygonGroupID = OutRawMesh.PolygonGroups().GetFirstValidID();
	}

	checkSlow(DstNumIndexes % 3 == 0);
	// Copy the vertices over
	TMap<int32, FVertexID> VertexIDMap;
	VertexIDMap.Reserve(DstNumPositions);
	{
		const TArray<FVector3f>& SrcPositions = SrcVertexDataMesh.Points;

		for (uint32 i = 0, I = DstNumPositions; i < I; ++i)
		{
			const FVector3f& Position = SrcPositions[i];
			const FVertexID NewVertexID = OutRawMesh.CreateVertex();
			VertexPositions[NewVertexID] = Position;
			VertexIDMap.Add(i, NewVertexID);
		}

		checkSlow(VertexPositions.GetNumElements() == DstNumPositions);
	}
	
	const bool bSrcHasTangentSpace = SrcVertexDataMesh.Tangent.Num() != 0 && SrcVertexDataMesh.BiTangent.Num() != 0 && SrcVertexDataMesh.Normal.Num() != 0;

	// Connectivity:
	auto CreateTriangle = [&OutRawMesh, &SrcVertexDataMesh, bSrcHasTangentSpace, &VertexIDMap, PolygonGroupID, &VertexInstanceNormals, &VertexInstanceTangents, &VertexInstanceBinormalSigns, &VertexInstanceColors, &VertexInstanceUVs, &EdgeHardnesses](const uint32 TriangleIndices[3])
	{
		int32 TriangleIndex = TriangleIndices[0] / 3;
		
		FVertexID VertexIndexes[3];
		TArray<FVertexInstanceID> VertexInstanceIDs;
		VertexInstanceIDs.SetNum(3);
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			int32 SrcIndex = SrcVertexDataMesh.Indices[TriangleIndices[Corner]];

			VertexIndexes[Corner] = VertexIDMap[SrcIndex];
			VertexInstanceIDs[Corner] = OutRawMesh.CreateVertexInstance(VertexIndexes[Corner]);

			//Tangents
			if (bSrcHasTangentSpace)
			{
				FVector3f Tangent = SrcVertexDataMesh.Tangent[SrcIndex];
				FVector3f BiTangent = SrcVertexDataMesh.BiTangent[SrcIndex];
				FVector3f Normal = SrcVertexDataMesh.Normal[SrcIndex];
				VertexInstanceTangents[VertexInstanceIDs[Corner]] = Tangent;
				VertexInstanceBinormalSigns[VertexInstanceIDs[Corner]] = GetBasisDeterminantSign((FVector)Tangent, (FVector)BiTangent, (FVector)Normal);
				VertexInstanceNormals[VertexInstanceIDs[Corner]] = Normal;
			}
			else
			{
				VertexInstanceTangents[VertexInstanceIDs[Corner]] = FVector3f(1, 0, 0);
				VertexInstanceNormals[VertexInstanceIDs[Corner]] = FVector3f(0, 0, 1);
				VertexInstanceBinormalSigns[VertexInstanceIDs[Corner]] = GetBasisDeterminantSign((FVector)VertexInstanceTangents[VertexInstanceIDs[Corner]].GetSafeNormal(),
																								 (FVector)(VertexInstanceNormals[VertexInstanceIDs[Corner]] ^ VertexInstanceTangents[VertexInstanceIDs[Corner]]).GetSafeNormal(),
																								 (FVector)VertexInstanceNormals[VertexInstanceIDs[Corner]].GetSafeNormal());
			}

			//Color
			if (SrcVertexDataMesh.FaceColors.Num() == 0)
			{
				VertexInstanceColors[VertexInstanceIDs[Corner]] = FVector4f(1.0f);
			}
			else
			{
				VertexInstanceColors[VertexInstanceIDs[Corner]] = FVector4f(FLinearColor(SrcVertexDataMesh.FaceColors[TriangleIndex]));
			}

			//UVs
			if (SrcVertexDataMesh.UVs.Num() == 0)
			{
				VertexInstanceUVs.Set(VertexInstanceIDs[Corner], FVector2f(0.0f, 0.0f));
			}
			else
			{
				VertexInstanceUVs.Set(VertexInstanceIDs[Corner], SrcVertexDataMesh.UVs[SrcIndex]);
			}
		}

		// Insert a polygon into the mesh
		OutRawMesh.CreatePolygon(PolygonGroupID, VertexInstanceIDs);
	};

	{
		uint32 RangeStop = DstNumIndexes / 3;
		for (uint32 i = 0, I = RangeStop; i < I; ++i)
		{
			uint32 SrcIndexes[3];
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				SrcIndexes[Corner] = (i * 3) + Corner;
			}
			CreateTriangle(SrcIndexes);
		}

		checkSlow(OutRawMesh.VertexInstances().Num() == DstNumIndexes);
	}

	//Put everybody in the same smoothgroup by default
	TArray<uint32> FaceSmoothingMasks;
	FaceSmoothingMasks.AddZeroed(SrcNumTriangles);

	if (SrcVertexDataMesh.FacePartition.Num() != 0)
	{
		for (uint32 FaceIndex = 0; FaceIndex < SrcNumTriangles; ++FaceIndex)
		{
			FaceSmoothingMasks[FaceIndex] = (1 << (SrcVertexDataMesh.FacePartition[FaceIndex] % 32));
		}
	}

	FStaticMeshOperations::ConvertSmoothGroupToHardEdges(FaceSmoothingMasks, OutRawMesh);
}


// Converts a raw mesh to a vertex data mesh.  This is potentially has some loss since the 
// raw mesh is nominally a per-index data structure and the vertex data mesh is a per-vertex structure.
// In addition, this only transfers the first texture coordinate and ignores material ids and vertex colors.

void ProxyLOD::RawMeshToVertexDataMesh(const FMeshDescription& SrcRawMesh, FVertexDataMesh& DstVertexDataMesh)
{
	FStaticMeshConstAttributes Attributes(SrcRawMesh);
	TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();


	const uint32 DstNumPositions = SrcRawMesh.Vertices().Num();

	uint32 DstNumIndexes = SrcRawMesh.Triangles().Num() * 3;

	// Copy the vertices over
	TMap<FVertexID, uint32> VertexIDToDstVertexIndex;
	VertexIDToDstVertexIndex.Reserve(SrcRawMesh.Vertices().Num());
	{
		// Allocate the space for the verts in the VertexDataMesh
		ResizeArray(DstVertexDataMesh.Points, DstNumPositions);
		uint32 VertexCount = 0;
		for (const FVertexID& VertexID : SrcRawMesh.Vertices().GetElementIDs())
		{
			DstVertexDataMesh.Points[VertexCount] = VertexPositions[VertexID];
			VertexIDToDstVertexIndex.Add(VertexID, VertexCount);
			VertexCount++;
		}
	}

	// Connectivity:
	uint32 VertexInstanceCount = 0;
	TArray<uint32>& DstIndices = DstVertexDataMesh.Indices;
	ResizeArray(DstIndices, DstNumIndexes);

	TArray<FVector3f>& DstTangentArray = DstVertexDataMesh.Tangent;
	ResizeArray(DstTangentArray, DstNumPositions);

	TArray<FVector3f>& DstBiTangentArray = DstVertexDataMesh.BiTangent;
	ResizeArray(DstBiTangentArray, DstNumPositions);

	TArray<FVector3f>& DstNormalArray = DstVertexDataMesh.Normal;
	ResizeArray(DstNormalArray, DstNumPositions);

	TArray<FVector2f>& DstUVs = DstVertexDataMesh.UVs;
	ResizeArray(DstUVs, DstNumPositions);

	//Iterate all triangle and add the indices
	for (const FTriangleID TriangleID : SrcRawMesh.Triangles().GetElementIDs())
	{
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			const FVertexInstanceID& VertexInstanceID = SrcRawMesh.GetTriangleVertexInstance(TriangleID, Corner);
			DstIndices[VertexInstanceCount] = VertexIDToDstVertexIndex[SrcRawMesh.GetVertexInstanceVertex(VertexInstanceID)];

			// Copy the tangent space:
			// NB: The tangent space is stored per-index in the raw mesh, but only per-vertex in the vertex data mesh.
			// We assume that the raw mesh per-index data is really duplicated per-vertex data!
			DstTangentArray[DstIndices[VertexInstanceCount]] = VertexInstanceTangents[VertexInstanceID];
			DstBiTangentArray[DstIndices[VertexInstanceCount]] = FVector3f::CrossProduct(VertexInstanceNormals[VertexInstanceID], VertexInstanceTangents[VertexInstanceID]).GetSafeNormal() * VertexInstanceBinormalSigns[VertexInstanceID];
			DstNormalArray[DstIndices[VertexInstanceCount]] = VertexInstanceNormals[VertexInstanceID];

			// Copy the UVs:
			// NB: The UVs is stored per-index in the raw mesh, but only per-vertex in the vertex data mesh.
			// We assume that the raw mesh per-index data is really duplicated per-vertex data!
			if (VertexInstanceUVs.GetNumChannels() == 0)
			{
				DstUVs[DstIndices[VertexInstanceCount]] = FVector2f(0.0f, 0.0f);
			}
			else
			{
				DstUVs[DstIndices[VertexInstanceCount]] = VertexInstanceUVs.Get(VertexInstanceID, 0);
			}

			VertexInstanceCount++;
		}
	}

	int32 NumTriangles = VertexInstanceCount / 3;
	TArray<uint32> FaceSmoothingMasks;
	FaceSmoothingMasks.AddZeroed(NumTriangles);

	FStaticMeshOperations::ConvertHardEdgesToSmoothGroup(SrcRawMesh, FaceSmoothingMasks);

	TArray<uint32>& DstFacePartition = DstVertexDataMesh.FacePartition;
	ResizeArray(DstFacePartition, NumTriangles);

	for (int32 FaceIndex = 0; FaceIndex < NumTriangles; ++FaceIndex)
	{
		DstFacePartition[FaceIndex] = 0;
		for (int32 BitIndex = 0; BitIndex < 32; ++BitIndex)
		{
			uint32 BitMask = FMath::Pow(2.f, BitIndex);
			DstFacePartition[FaceIndex] += ((FaceSmoothingMasks[FaceIndex] & BitMask) > 0);
		}
	}
}




template <typename  AOSVertexType>
static void CopyIndexAndPos(const TAOSMesh<AOSVertexType>& AOSMesh, FVertexDataMesh& VertexDataMesh)
{

	const uint32 DstNumPositions = AOSMesh.GetNumVertexes();
	const uint32 DstNumIndexes = AOSMesh.GetNumIndexes();

	checkSlow(DstNumIndexes % 3 == 0);

	TArray<uint32>& WedgeIndices = VertexDataMesh.Indices;
	ResizeArray(VertexDataMesh.Points, DstNumPositions);
	ResizeArray(WedgeIndices, DstNumIndexes);


	// Copy the vertices over
	{
		ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, DstNumPositions),
			[&VertexDataMesh, &AOSMesh](const ProxyLOD::FUIntRange& Range)
		{
			FVector3f* Points = VertexDataMesh.Points.GetData();
			for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				const FVector3f& Position = AOSMesh.Vertexes[i].GetPos();
				Points[i] = Position;
			}
		}
		);

	}


	//  Connectivity
	{
		ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, DstNumIndexes),
			[&AOSMesh, &WedgeIndices](const ProxyLOD::FUIntRange& Range)
		{
			for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				WedgeIndices[i] = AOSMesh.Indexes[i];
			}
		}
		);

		checkSlow(WedgeIndices.Num() == DstNumIndexes);
	}

}

template <typename  AOSVertexType>
static void CopyNormals(const TAOSMesh<AOSVertexType>& AOSMesh, FVertexDataMesh& VertexDataMesh)
{

	// Copy the tangent space attributes.

	const uint32 DstNumPositions = AOSMesh.GetNumVertexes();

	checkSlow(AOSMesh.GetNumIndexes() % 3 == 0);

	TArray<FVector3f>& NormalArray = VertexDataMesh.Normal;
	ResizeArray(NormalArray, DstNumPositions);


	// Transfer the normal

	{
		ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, DstNumPositions),
			[&AOSMesh, &NormalArray](const ProxyLOD::FUIntRange& Range)
		{
			for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				const auto& AOSVertex = AOSMesh.Vertexes[i];
				NormalArray[i] = AOSVertex.Normal;
			}
		}
		);
	}
}


// Populate a VertexDataMesh with the information in the Array of Structs mesh.
template <typename  AOSVertexType>
static void AOSMeshToVertexDataMesh(const TAOSMesh<AOSVertexType>& AOSMesh, FVertexDataMesh& VertexDataMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AOSMeshToVertexDataMesh)

	// Copy the topology and geometry of the mesh

	CopyIndexAndPos(AOSMesh, VertexDataMesh);

	// adds t = (1,0,0)  bt = (0, 1, 0)  n = (0, 0, 1)
	ProxyLOD::AddDefaultTangentSpace(VertexDataMesh);

	// Copy the tangent space attributes.

	CopyNormals(AOSMesh, VertexDataMesh);
}


// The posistion only specialization only adds a default tangent space
template <>
void AOSMeshToVertexDataMesh<FPositionOnlyVertex>(const TAOSMesh<FPositionOnlyVertex>& AOSMesh, FVertexDataMesh& VertexDataMesh)
{

	// Copy the topology and geometry of the mesh

	CopyIndexAndPos(AOSMesh, VertexDataMesh);


	// adds t = (1,0,0)  bt = (0, 1, 0)  n = (0, 0, 1)
	ProxyLOD::AddDefaultTangentSpace(VertexDataMesh);
}


void ProxyLOD::ConvertMesh(const FAOSMesh& InMesh, FVertexDataMesh& OutMesh)
{
	AOSMeshToVertexDataMesh(InMesh, OutMesh);
}


void ProxyLOD::ConvertMesh(const FAOSMesh& InMesh, FMeshDescription& OutMesh)
{
	AOSMeshToRawMesh(InMesh, OutMesh);
}

void ProxyLOD::ConvertMesh(const FVertexDataMesh& InMesh, FMeshDescription& OutMesh)
{
	VertexDataMeshToRawMesh(InMesh, OutMesh);
}

void ProxyLOD::ConvertMesh(const FMeshDescription& InMesh, FVertexDataMesh& OutMesh)
{
	RawMeshToVertexDataMesh(InMesh, OutMesh);
}

