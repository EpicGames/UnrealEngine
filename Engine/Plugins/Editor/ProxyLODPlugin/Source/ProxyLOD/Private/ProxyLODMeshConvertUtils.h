// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProxyLODMeshTypes.h"
#include "ProxyLODThreadedWrappers.h"

#include "Math/Vector4.h"




/** 
* Utilities to convert between various mesh types.  Only the minimum of geometric conversion is guaranteed.
*  
* NB: Many of the conversions are lossy, since the mesh types have different attributes and frequency.
*
*/
namespace ProxyLOD
{

	/**
	* Convert a mixed triangle and quad mesh to a triangle mesh type by splitting quads.
	* No new vertices are created.
	*
	* NB: Default valued attributes are added to the resulting mesh, e.g. tangent space, colors, etc
	*
	* @param InMesh   Mesh with a mixture of quads and triangles.
	* @param OutMesh  Triangle Mesh.
	*/
	void MixedPolyMeshToRawMesh( const FMixedPolyMesh& InMesh, FMeshDescription& OutMesh );
	
	/**
	* Convert a mixed triangle and quad mesh to a triangle mesh type by splitting quads.
	* No new vertices are created.
	*
	* NB: This does not attempt to add any additional attributes to the result.  Will
	*     need to separately compute vertex normals if desired.
	*
	* @param InMesh   Mesh with a mixture of quads and triangles.
	* @param OutMesh  Triangle Mesh.
	* @param bClockWise how to order the verts in a triangle
	*/
	template <typename T>
	void MixedPolyMeshToAOSMesh(const FMixedPolyMesh& InMesh, TAOSMesh<T>& OutMesh, const bool bClockWise = true);


	/**
	* Common interface to convert between various triangle-based mesh types.
	*
	* The conversions will maintain geometry and connectivity, but different mesh types support
	* different attribute types and possibly different frequenceies even when the same attribute types
	* exist on both mesh types ( e.g. FMeshDescription has per-wedge data while FVertexDataMesh has per-vertex attributes).
	*
	* @param InMesh   Source Mesh to convert.
	* @param OutMesh  The result of the mesh conversion.  Any data already stored in OutMesh will be lost.
	*
	*/
	void ConvertMesh(const FAOSMesh& InMesh, FMeshDescription& OutMesh);
	void ConvertMesh(const FAOSMesh& InMesh, FVertexDataMesh& OutMesh);
	void ConvertMesh(const FVertexDataMesh& InMesh, FMeshDescription& OutMesh);
	void ConvertMesh(const FMeshDescription& InMesh, FVertexDataMesh& OutMesh);

	
	/**
	* Convert the simplifier-friendly array-of-structs mesh to a struct of arrays FMeshDescription.
	*
	* NB: This copies the AOSVertex normal to the wedge FMeshDescription::Tangentz.
	*     But additional FMeshDescription attributes, including tangent/bitangent are given default values.
	*
	* @param InMesh   Source Mesh to convert.
	* @param OutMesh  The result of the mesh conversion.  Any data already stored in OutMesh will be lost.
	*
	*/
	void AOSMeshToRawMesh( const FAOSMesh& InMesh, FMeshDescription& OutMesh );


	/**
	* Convert the uv-generation-friendly vertex data mesh to a struct of arrays FMeshDescription.
	* In addition to connectivity and vertex locations, this also transfers tangent space
	* and UVs.
	*
	* @param InMesh   Source Mesh to convert.
	* @param OutMesh  The result of the mesh conversion.  Any data already stored in OutMesh will be lost.
	*
	*/
	void VertexDataMeshToRawMesh( const FVertexDataMesh& InMesh, FMeshDescription& OutMesh );

	/**
	* Converts a FMeshDescription to a  uv-generation-friendly vertex data mesh.  This is potentially has some loss since the 
	* raw mesh is nominally a per-index data structure and the vertex data mesh is a per-vertex structure.
	* In addition, this only transfers the first texture coordinate and ignores material ids and vertex colors.
	*
	* @param InMesh   Source Mesh to convert.
	* @param OutMesh  The result of the mesh conversion.  Any data already stored in OutMesh will be lost.
	*
	*/
	void RawMeshToVertexDataMesh( const FMeshDescription& InMesh, FVertexDataMesh& OutMesh );

}


#ifndef PROXYLOD_CLOCKWISE_TRIANGLES
#define PROXYLOD_CLOCKWISE_TRIANGLES  1
#endif

static FVector3f ComputeNormal(const FVector3f(&Tri)[3])
{

	FVector3f N = FVector3f::CrossProduct(Tri[1] - Tri[0], Tri[2] - Tri[0]);
	N.Normalize();
#if	(PROXYLOD_CLOCKWISE_TRIANGLES == 1)
	return -N;
#else
	return N;
#endif 
}

// Convert MixedPolyMesh to AOS Mesh.  This requires splitting quads to produce triangles.
template <typename T>
void ProxyLOD::MixedPolyMeshToAOSMesh(const FMixedPolyMesh& MixedPolyMesh, TAOSMesh<T>& DstAOSMesh, bool bClockWise)
{

	// Splitting a quad doesn't introduce any new verts.
	const uint32 DstNumVerts = MixedPolyMesh.Points.size();

	const uint32 NumQuads = MixedPolyMesh.Quads.size();

	// Each quad becomes 2 triangles.
	const uint32 DstNumTris = 2 * NumQuads + MixedPolyMesh.Triangles.size();

	// Each Triangle has 3 corners
	const uint32 DstNumIndexes = 3 * DstNumTris;

	// Empty and Allocate space

	DstAOSMesh.Resize(DstNumVerts, DstNumTris);

	// Copy the vertices position over and give it a dummy material index.
	{
		// Allocate the space for the verts in the DstAOSMesh

		ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, DstNumVerts),
			[&MixedPolyMesh, &DstAOSMesh](const ProxyLOD::FUIntRange& Range)
		{
			for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				const openvdb::Vec3s& Point = MixedPolyMesh.Points[i];
				DstAOSMesh.Vertexes[i].Position = FVector3f(Point[0], Point[1], Point[2]);
				DstAOSMesh.Vertexes[i].MaterialIndex = 0;
			}
		});
	}

	// Split VDB Quads
	{


		// NB: The Quads are ordered in clockwise fashion.

		ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumQuads),
			[&MixedPolyMesh, &DstAOSMesh, bClockWise](const ProxyLOD::FUIntRange& Range)
		{
			uint32* Indices = DstAOSMesh.Indexes;
			for (uint32 q = Range.begin(), Q = Range.end(); q < Q; ++q)
			{
				const uint32 Offset = q * 6;
				const openvdb::Vec4I& Quad = MixedPolyMesh.Quads[q];
				// add as two triangles
				if (bClockWise)
				{
					// first triangle
					Indices[Offset] = Quad[0];
					Indices[Offset + 1] = Quad[1];
					Indices[Offset + 2] = Quad[2];
					// second triangle
					Indices[Offset + 3] = Quad[2];
					Indices[Offset + 4] = Quad[3];
					Indices[Offset + 5] = Quad[0];
				}
				else
				{
					// first triangle
					Indices[Offset] = Quad[0];
					Indices[Offset + 1] = Quad[3];
					Indices[Offset + 2] = Quad[2];

					// second triangle
					Indices[Offset + 3] = Quad[2];
					Indices[Offset + 4] = Quad[1];
					Indices[Offset + 5] = Quad[0];
				}
			}

		});

		// add the MixedPolyMesh triangles.
		ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, MixedPolyMesh.Triangles.size()),
			[&MixedPolyMesh, &DstAOSMesh, NumQuads, bClockWise](const ProxyLOD::FUIntRange& Range)
		{
			uint32* Indices = DstAOSMesh.Indexes;
			for (uint32 t = Range.begin(), EndT = Range.end(); t < EndT; ++t)
			{
				const uint32 Offset = NumQuads * 6 + t * 3;
				const openvdb::Vec3I& Tri = MixedPolyMesh.Triangles[t];
				// add the triangle
				if (bClockWise)
				{
					Indices[Offset] = Tri[0];
					Indices[Offset + 1] = Tri[1];
					Indices[Offset + 2] = Tri[2];
				}
				else
				{
					Indices[Offset] = Tri[2];
					Indices[Offset + 1] = Tri[1];
					Indices[Offset + 2] = Tri[0];
				}
			}
		});
	}
}
