// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODMeshParameterization.h"
#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START
#include <UVAtlas.h>
#include <DirectXMesh/DirectXMesh.h>
THIRD_PARTY_INCLUDES_END


#include "ProxyLODThreadedWrappers.h"
#include "ProxyLODMeshUtilities.h"


bool ProxyLOD::GenerateUVs(const FTextureAtlasDesc& TextureAtlasDesc,
	                       const TArray<FVector3f>&   VertexBuffer, 
	                       const TArray<int32>&     IndexBuffer, 
						   const TArray<int32>&     AdjacencyBuffer,
	                       TFunction<bool(float)>&  Callback, 
	                       TArray<FVector2D>&       UVVertexBuffer, 
	                       TArray<int32>&           UVIndexBuffer, 
	                       TArray<int32>&           VertexRemapArray, 
	                       float&                   MaxStretch, 
	                       int32&                   NumCharts)
{
	const int32 NumVerts   = VertexBuffer.Num();
	const int32 NumFaces   = IndexBuffer.Num() / 3;

	// Copy data into DirectX library format

	TArray<DirectX::XMFLOAT3> PosArray;
	PosArray.Empty(NumVerts);
	for (int32 i = 0; i < NumVerts; ++i)
	{
		const FVector3f& Vertex = VertexBuffer[i];
		PosArray.Emplace(DirectX::XMFLOAT3(Vertex.X, Vertex.Y, Vertex.Z));
	}
	
	TArray<uint32_t> Indices;
	Indices.Empty(3 * NumFaces);
	for (int32 i = 0, I = NumFaces * 3; i < I; ++i)
	{
		Indices.Add(IndexBuffer[i]);
	}

	TArray<uint32_t> AdjacencyArray;
	AdjacencyArray.Empty(3 * NumFaces);
	for (int32 i = 0, I = NumFaces * 3; i < I; ++i)
	{
		AdjacencyArray.Add( static_cast<uint32_t>(AdjacencyBuffer[i]) );
	}



	// Verify the mesh is valid
	{
	
		HRESULT ValidateHR = DirectX::Validate(Indices.GetData(), NumFaces, NumVerts, AdjacencyArray.GetData(), DirectX::VALIDATE_DEFAULT, NULL);
		if (FAILED(ValidateHR))
		{
			return false;
		}
	}

	// Capture the callback and convert result type
	auto StatusCallBack = [&Callback](float percentComplete)->HRESULT
	{
		//return S_OK;
		return (Callback(percentComplete)) ? S_OK : S_FALSE;
	};

	

	// Translate controls into corret types
	const size_t NumChartsIn  = (size_t)NumCharts;
	const float  MaxStretchIn = MaxStretch;

	const size_t Width  = TextureAtlasDesc.Size.X;
	const size_t Height = TextureAtlasDesc.Size.Y;
	const float  Gutter = TextureAtlasDesc.Gutter;

	// Symmetric Identity matrix used for the signal
	std::vector<float> IMTArray(NumFaces * 3);
	{
		for (int32 f = 0; f < NumFaces; ++f)
		{
			int32 offset = 3 * f;
			{
				IMTArray[offset + 0] = 1.f;
				IMTArray[offset + 1] = 0.f;
				IMTArray[offset + 2] = 1.f;
			}
		}
	}


	size_t NumChartsOut = 0;
	float MaxStretchOut = 0.f;
	
	// info to capture
	std::vector<DirectX::UVAtlasVertex> VB;
	std::vector<uint8>  IB;
	std::vector<uint32> RemapArray;
	std::vector<uint32> FacePartitioning;

	// Generate UVs
	constexpr DXGI_FORMAT IndexFormat = DXGI_FORMAT_R32_UINT;

	std::vector<uint32_t> vPartitionResultAdjacency;

	HRESULT hr = DirectX::UVAtlasPartition(PosArray.GetData(), NumVerts,
		Indices.GetData(), IndexFormat, NumFaces,
		NumChartsIn, MaxStretchIn,
		AdjacencyArray.GetData(), NULL /*false adj*/, IMTArray.data() /*IMTArray*/,
		StatusCallBack, DirectX::UVATLAS_DEFAULT_CALLBACK_FREQUENCY,
		DirectX::UVATLAS_DEFAULT, VB, IB,
		&FacePartitioning, &RemapArray, vPartitionResultAdjacency, &MaxStretchOut, &NumChartsOut);

	if (hr != S_OK)
	{
		return false;
	}

	if (!ensure(vPartitionResultAdjacency.size() == 3 * NumFaces))
	{
		return false;
	}

	//
	// UVAtlas has a bug where it can crash if multiple charts reference the same vertex index, depending on the order in which the charts are processed internally. 
	// (e.g. while processing chart[i], it assumes all vertex indices are in some range [min[i], max[i]]. If it tries to reuse a vertex index less than min[i], it may crash.)
	// To workaround this, we will duplicate shared vertices into their own charts.
	//

	std::vector<bool> bUsedFace(NumFaces, false);

	std::vector<DirectX::UVAtlasVertex> NewVertexBuffer;

	static_assert(IndexFormat == DXGI_FORMAT_R32_UINT);
	check(IB.size() % sizeof(uint32_t) == 0);

	std::vector<uint32_t> IndexBufferU32;
	IndexBufferU32.resize(IB.size() / sizeof(uint32_t));
	FMemory::Memcpy(IndexBufferU32.data(), IB.data(), IndexBufferU32.size() * sizeof(uint32_t));

	std::vector<uint32_t> NewIndexBufferU32;
	NewIndexBufferU32.resize(3 * NumFaces);

	std::vector<uint32> NewRemapArray;
	NewRemapArray.resize(NumVerts, -1);

	// Build partioning based on triangle adjacency (this is how UVAtlas will initialize things internally, NOT via the output FacePartitioning from UVAtlasPartition as one might expect)

	for (int32 FaceIndex = 0; FaceIndex < NumFaces; FaceIndex++)
	{
		if (!bUsedFace[FaceIndex])
		{
			// new chart
			bUsedFace[FaceIndex] = true;

			std::vector<uint32_t> CurrentChartTriangles;
			CurrentChartTriangles.push_back(FaceIndex);

			size_t ChartTriangleIndex = 0;

			// use breadth-first search to find chart
			while (ChartTriangleIndex < CurrentChartTriangles.size())
			{
				for (uint32_t TriEdgeIndex = 0; TriEdgeIndex < 3; TriEdgeIndex++)
				{
					const uint32_t TriangleAdjacencyIndex = 3 * CurrentChartTriangles[ChartTriangleIndex] + TriEdgeIndex;
					const uint32_t AdjacentTriangleIndex = vPartitionResultAdjacency[TriangleAdjacencyIndex];
					if (AdjacentTriangleIndex != -1 && !bUsedFace[AdjacentTriangleIndex])
					{
						CurrentChartTriangles.push_back(AdjacentTriangleIndex);
						bUsedFace[AdjacentTriangleIndex] = true;
					}
				}
				ChartTriangleIndex++;
			}

			// Now rebuild the vertex and index buffers for this chart. Vertices that appear in multiple charts should be duplicated by this process.
			const size_t ChartMinIndex = NewVertexBuffer.size();
			TMap<uint32_t, uint32_t> VertexMap;

			for (uint32_t CurrentChartFaceIndex : CurrentChartTriangles)
			{
				for (int32 TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
				{
					const uint32_t TriVert = IndexBufferU32[3 * CurrentChartFaceIndex + TriVertIndex];

					uint32_t NewVertexIndex;
					if (uint32_t* MappedVertex = VertexMap.Find(TriVert))
					{
						NewVertexIndex = *MappedVertex;
					}
					else
					{
						NewVertexIndex = (uint32_t)NewVertexBuffer.size();
						NewVertexBuffer.push_back(VB[TriVert]);
						VertexMap.Add(TriVert, NewVertexIndex);
					}

					if (NewRemapArray.size() <= NewVertexIndex)
					{
						NewRemapArray.resize(NewVertexIndex + 1);
					}
					NewRemapArray[NewVertexIndex] = RemapArray[TriVert];

					ensure(ChartMinIndex <= NewVertexIndex);

					NewIndexBufferU32[3 * CurrentChartFaceIndex + TriVertIndex] = NewVertexIndex;
				}
			}
		}
	}

	std::vector<uint8> NewIndexBuffer;
	NewIndexBuffer.resize(NewIndexBufferU32.size() * sizeof(uint32_t));
	FMemory::Memcpy(NewIndexBuffer.data(), NewIndexBufferU32.data(), NewIndexBuffer.size());

	hr = DirectX::UVAtlasPack(NewVertexBuffer, NewIndexBuffer, DXGI_FORMAT_R32_UINT,
		Width, Height, Gutter,
		vPartitionResultAdjacency,
		StatusCallBack, DirectX::UVATLAS_DEFAULT_CALLBACK_FREQUENCY);


	// Translate results to output form.
	if (hr == S_OK)
	{
		const int32 NumUVverts    = (int32)NewVertexBuffer.size();
		const int32 NumUVindices  = (int32)NewIndexBuffer.size();
		const int32 NumRemapArray = (int32)NewRemapArray.size();

		// empty and resize

		UVVertexBuffer.Empty(NumUVverts);
		UVIndexBuffer.Empty(NumUVindices);
		VertexRemapArray.Empty(NumRemapArray);

		for (const DirectX::UVAtlasVertex& UVvertex : NewVertexBuffer)
		{
			const auto& UV = UVvertex.uv;
			UVVertexBuffer.Emplace(FVector2D(UV.x, UV.y));
		}

		// This part is weird
		{
			std::vector<uint32> indices;
			indices.resize(3 * NumFaces);
			std::memcpy(indices.data(), NewIndexBuffer.data(), sizeof(uint32) * 3 * NumFaces);

			for (uint32 i : indices)
			{
				UVIndexBuffer.Add(i);
			}
		}

		for (const uint32 i : NewRemapArray)
		{
			int32 r = i;
			VertexRemapArray.Add(r);
		}

		MaxStretch = MaxStretchOut;
		NumCharts  = (int32)NumChartsOut;
	}
	

	return (hr == S_OK) ? true : false;
}


bool ProxyLOD::GenerateUVs(FVertexDataMesh& InOutMesh, const FTextureAtlasDesc& TextureAtlasDesc, const bool VertexColorParts)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::GenerateUVs)

	// desired parameters for ISO-Chart method

	// MaxChartNum = 0 will allow any number of charts to be generated.

	const size_t MaxChartNumber = 0;

	// Let the polys in the partitions stretch some..  1.f will let it stretch freely

	const float MaxStretch = 0.125f; // Question.  Does this override the pIMT?
	
	// Note: I tried constructing this from the normals, but that resulted in some large planar regions being really compressed in the
	// UV chart.
	const bool bComputeIMTFromVertexNormal = false;

	// No Op
	auto NoOpCallBack = [](float percent)->HRESULT {return  S_OK; };

	return GenerateUVs(InOutMesh, TextureAtlasDesc, VertexColorParts, MaxStretch, MaxChartNumber, bComputeIMTFromVertexNormal, NoOpCallBack);
}

bool ProxyLOD::GenerateUVs(FVertexDataMesh& InOutMesh, const FTextureAtlasDesc& TextureAtlasDesc, const bool VertexColorParts, 
	                      const float MaxStretch, const size_t MaxChartNumber, const bool bComputeIMTFromVertexNormal, 
	                      std::function<HRESULT __cdecl(float percentComplete)> StatusCallBack, float* MaxStretchOut, size_t* NumChartsOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::GenerateUVs)

	std::vector<uint32> DirextXAdjacency;

	const bool bValidMesh = GenerateAdjacenyAndCleanMesh(InOutMesh, DirextXAdjacency);

	if (!bValidMesh) return false;


	// Data from the existing mesh

	const DirectX::XMFLOAT3* Pos = (DirectX::XMFLOAT3*) (InOutMesh.Points.GetData());
	const size_t NumVerts = InOutMesh.Points.Num();
	const size_t NumFaces = InOutMesh.Indices.Num() / 3;
	uint32* indices = InOutMesh.Indices.GetData();



	// The mesh adjacency

	const uint32* adjacency = DirextXAdjacency.data();


	// Size of the texture atlas

	const size_t width = TextureAtlasDesc.Size.X;
	const size_t height = TextureAtlasDesc.Size.Y;
	const float gutter = TextureAtlasDesc.Gutter;


	// Partition and mesh info to capture

	std::vector<DirectX::UVAtlasVertex> vb;
	std::vector<uint8> ib;
	std::vector<uint32> vertexRemapArray;
	std::vector<uint32> facePartitioning;

	// Capture stats about the result.
	float maxStretchUsed = 0.f;
	size_t numChartsUsed = 0;

	TArray<float> IMTArray;
	IMTArray.SetNumUninitialized(NumFaces * 3);
	float* pIMTArray = IMTArray.GetData();

	if (!bComputeIMTFromVertexNormal)
	{
		for (int32 f = 0; f < NumFaces; ++f)
		{
			int32 offset = 3 * f;
			{
				pIMTArray[offset] = 1.f;
				pIMTArray[offset + 1] = 0.f;
				pIMTArray[offset + 2] = 1.f;
			}
		}
	}
	else
	{
		// per-triangle IMT from per-vertex data(normal). 

		const TArray<FVector3f>& Normals = InOutMesh.Normal;
		const float* PerVertSignal = (float*)Normals.GetData();
		size_t SignalStride = 3 * sizeof(float);
		HRESULT IMTResult = UVAtlasComputeIMTFromPerVertexSignal(Pos, NumVerts, indices, DXGI_FORMAT_R32_UINT, NumFaces, PerVertSignal, 3, SignalStride, StatusCallBack, pIMTArray);

		if (FAILED(IMTResult))
		{
			return false;
		}
	}

	std::vector<uint32_t> vPartitionResultAdjacency;
	HRESULT hr;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DirectX::UVAtlasPartition)
		hr = DirectX::UVAtlasPartition(Pos, NumVerts, 
			indices, DXGI_FORMAT_R32_UINT, NumFaces,
			MaxChartNumber, MaxStretch,
			adjacency, NULL /*false adj*/, pIMTArray,
			StatusCallBack, DirectX::UVATLAS_DEFAULT_CALLBACK_FREQUENCY,
			DirectX::UVATLAS_DEFAULT, vb, ib,
			&facePartitioning, &vertexRemapArray,
			vPartitionResultAdjacency,
			&maxStretchUsed, &numChartsUsed);
	}

	if (SUCCEEDED(hr))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DirectX::UVAtlasPack)
		hr = DirectX::UVAtlasPack(
			vb, ib,
			DXGI_FORMAT_R32_UINT,
			width, height, gutter,
			vPartitionResultAdjacency,
			StatusCallBack, DirectX::UVATLAS_DEFAULT_CALLBACK_FREQUENCY);
	}

	if (MaxStretchOut)
	{
		*MaxStretchOut = maxStretchUsed;
	}
	if (NumChartsOut)
	{
		*NumChartsOut = numChartsUsed;
	}

	if (FAILED(hr))
	{
		return false;
	}

	// testing
	check(ib.size() / sizeof(uint32) == NumFaces * 3);
	check(facePartitioning.size() == NumFaces);
	check(vertexRemapArray.size() == vb.size());

	// The mesh partitioning may split vertices, and this needs to be reflected in the mesh.

	// Update Faces
	{
		std::memcpy(indices, ib.data(), sizeof(uint32) * 3 * NumFaces);
	}

	// Add UVs
	{
		const size_t NumNewVerts = vb.size();
		ResizeArray(InOutMesh.UVs, NumNewVerts);

		FVector2f* UVCoords = InOutMesh.UVs.GetData();
		size_t j = 0;
		for (auto it = vb.cbegin(); it != vb.cend() && j < NumNewVerts; ++it, ++j)
		{
			const auto& UV = it->uv;
			UVCoords[j] = FVector2f(UV.x, UV.y);
		}

	}

	ProxyLOD::FTaskGroup TaskGroup;


	TaskGroup.Run([&]()
	{
		const size_t NumNewVerts = vb.size();
		bool bReducedVertCount = NumNewVerts < NumVerts;
		check(!bReducedVertCount);

		// Copy the New Verts into a TArray so we can do a swap.
		TArray<FVector3f> NewVertArray;
		ResizeArray(NewVertArray, NumNewVerts);

		// re-order the verts 
		DirectX::UVAtlasApplyRemap(InOutMesh.Points.GetData(), sizeof(FVector3f), NumVerts, NumNewVerts, vertexRemapArray.data(), NewVertArray.GetData());

		// swap the data into the raw mesh 
		Swap(NewVertArray, InOutMesh.Points);
	});

	// Update the normals
	TaskGroup.Run([&]()
	{
		const size_t NumNewVerts = vb.size();
		TArray<FVector3f> NewNormalsArray;
		ResizeArray(NewNormalsArray, NumNewVerts);

		// re-order the verts 
		DirectX::UVAtlasApplyRemap(InOutMesh.Normal.GetData(), sizeof(FVector3f), NumVerts, NumNewVerts, vertexRemapArray.data(), NewNormalsArray.GetData());

		// swap the data into the raw mesh 
		Swap(NewNormalsArray, InOutMesh.Normal);

	});

	// Update the transfer normals
	TaskGroup.Run([&]()
	{
		const size_t NumNewVerts = vb.size();
		TArray<FVector3f> NewTransferNormalsArray;
		ResizeArray(NewTransferNormalsArray, NumNewVerts);

		// re-order the verts 
		DirectX::UVAtlasApplyRemap(InOutMesh.TransferNormal.GetData(), sizeof(FVector3f), NumVerts, NumNewVerts, vertexRemapArray.data(), NewTransferNormalsArray.GetData());

		// swap the data into the raw mesh 
		Swap(NewTransferNormalsArray, InOutMesh.TransferNormal);

	});

	// Update the tangents
	TaskGroup.Run([&]()
	{
		const size_t NumNewVerts = vb.size();
		TArray<FVector3f> NewTangentArray;
		ResizeArray(NewTangentArray, NumNewVerts);

		// re-order the verts 
		DirectX::UVAtlasApplyRemap(InOutMesh.Tangent.GetData(), sizeof(FVector3f), NumVerts, NumNewVerts, vertexRemapArray.data(), NewTangentArray.GetData());

		// swap the data into the raw mesh 
		Swap(NewTangentArray, InOutMesh.Tangent);

	});

	TaskGroup.Run([&]()
	{
		const size_t NumNewVerts = vb.size();
		TArray<FVector3f> NewBiTangentArray;
		ResizeArray(NewBiTangentArray, NumNewVerts);

		// re-order the verts 
		DirectX::UVAtlasApplyRemap(InOutMesh.BiTangent.GetData(), sizeof(FVector3f), NumVerts, NumNewVerts, vertexRemapArray.data(), NewBiTangentArray.GetData());

		// swap the data into the raw mesh 
		Swap(NewBiTangentArray, InOutMesh.BiTangent);

	});

	TaskGroup.Wait();

	ResizeArray(InOutMesh.FacePartition, NumFaces);

	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumFaces),
		[&InOutMesh, &facePartitioning](const ProxyLOD::FIntRange& Range)
	{
		auto& FacePartition = InOutMesh.FacePartition;
		for (int32 f = Range.begin(), F = Range.end(); f < F; ++f)
		{
			FacePartition[f] = facePartitioning[f];
		}
	});

	if (VertexColorParts)
	{
		// Color the verts by partition for debuging.

		ColorPartitions(InOutMesh, facePartitioning);
	}

	return true;
}


void ProxyLOD::GenerateAdjacency(const FAOSMesh& AOSMesh, std::vector<uint32>& AdjacencyArray)
{
	const uint32 NumTris = AOSMesh.GetNumIndexes() / 3;
	const uint32 AdjacencySize = AOSMesh.GetNumIndexes(); // = 3 for each face.

														  // Get the positions as a single array.
	std::vector<FVector3f> PosArray;
	AOSMesh.GetPosArray(PosArray);

	// Allocate adjacency
	AdjacencyArray.resize(AdjacencySize);

	// position comparison epsilon
	const float Eps = 0.f;
	HRESULT hr = DirectX::GenerateAdjacencyAndPointReps(AOSMesh.Indexes, NumTris, (DirectX::XMFLOAT3*)PosArray.data(), PosArray.size(), Eps, NULL /* optional point rep pointer*/, AdjacencyArray.data());

}

void ProxyLOD::GenerateAdjacency(const FVertexDataMesh& Mesh, std::vector<uint32>& AdjacencyArray)
{
	const uint32 NumTris = Mesh.Indices.Num() / 3;
	const uint32 AdjacencySize = Mesh.Indices.Num(); // = 3 for each face.

													 // Get the positions as a single array.
	const FVector3f* PosArray = Mesh.Points.GetData();
	const uint32 NumPos = Mesh.Points.Num();

	// Allocate adjacency
	AdjacencyArray.resize(AdjacencySize);

	// position comparison epsilon
	const float Eps = 0.f;
	HRESULT hr = DirectX::GenerateAdjacencyAndPointReps(Mesh.Indices.GetData(), NumTris, (const DirectX::XMFLOAT3*)PosArray, NumPos, Eps, NULL /* optional point rep pointer*/, AdjacencyArray.data());

}
void ProxyLOD::GenerateAdjacency(const FMeshDescription& RawMesh, std::vector<uint32>& AdjacencyArray)
{
	uint32 NumTris = RawMesh.Triangles().Num();
	const uint32 NumVerts = RawMesh.Vertices().Num();
	const uint32 AdjacencySize = RawMesh.VertexInstances().Num(); // 3 for each face

															 // Allocate adjacency
	AdjacencyArray.resize(AdjacencySize);

	// @todo: possibility to pass the vertex position raw array directly into the below function.
	// Can it be assumed that it is not sparse? 
	TArrayView<const FVector3f> VertexPositionsAttribute = RawMesh.GetVertexPositions().GetRawArray();
	TArray<FVector3f> VertexPositions;
	VertexPositions.AddZeroed(NumVerts);
	for (const FVertexID VertexID : RawMesh.Vertices().GetElementIDs())
	{
		VertexPositions[VertexID.GetValue()] = VertexPositionsAttribute[VertexID];
	}

	TArray<uint32> Indices;
	Indices.AddZeroed(AdjacencySize);

	for (const FVertexInstanceID VertexInstanceID : RawMesh.VertexInstances().GetElementIDs())
	{
		Indices[VertexInstanceID.GetValue()] = RawMesh.GetVertexInstanceVertex(VertexInstanceID).GetValue();
	}
	// position comparison epsilon
	const float Eps = 0.f;
	HRESULT hr = DirectX::GenerateAdjacencyAndPointReps(Indices.GetData(), NumTris, (DirectX::XMFLOAT3*)VertexPositions.GetData(), NumVerts, Eps, NULL /* optional point rep pointer*/, AdjacencyArray.data());
}

bool ProxyLOD::GenerateAdjacenyAndCleanMesh(FVertexDataMesh& InOutMesh, std::vector<uint32>& Adjacency)
{
	std::vector<uint32_t> dupVerts;

	uint32 CleanCount = 0;
	while (CleanCount == 0 || (dupVerts.size() != 0 && CleanCount < 5))
	{
		CleanCount++;

		// Rebuild the adjacency.  
		Adjacency.clear();
		GenerateAdjacency(InOutMesh, Adjacency);

		dupVerts.clear();
		HRESULT hr = DirectX::Clean(InOutMesh.Indices.GetData(), InOutMesh.Indices.Num() / 3, InOutMesh.Points.Num(), Adjacency.data(), NULL, dupVerts, true /*break bowties*/);

		SplitVertices(InOutMesh, dupVerts);

		uint32 Offset = InOutMesh.Points.Num();

		// spatially separate bowties
		for (int32 f = 0; f < InOutMesh.Indices.Num() / 3; ++f)
		{
			for (int32 v = 0; v < 3; ++v)
			{
				const uint32 Idx = InOutMesh.Indices[3 * f + v];

				// This is a duplicate vert, find it's triangle and push it towards the center.
				if (Idx > Offset - 1)
				{
					const uint32 TriIds[3] = { InOutMesh.Indices[3 * f], InOutMesh.Indices[3 * f + 1], InOutMesh.Indices[3 * f + 2] };

					// compute center of this face
					FVector3f CenterOfFace = InOutMesh.Points[TriIds[0]] + InOutMesh.Points[TriIds[1]] + InOutMesh.Points[TriIds[2]];
					CenterOfFace /= 3.f;

					// Vector to center
					FVector3f PointToCenter = (InOutMesh.Points[Idx] - CenterOfFace);
					PointToCenter.Normalize();

					// move vert towards center
					InOutMesh.Points[Idx] = InOutMesh.Points[Idx] - 0.0001f * PointToCenter;
				}
			}
		}
	}

	return (dupVerts.size() == 0);

}