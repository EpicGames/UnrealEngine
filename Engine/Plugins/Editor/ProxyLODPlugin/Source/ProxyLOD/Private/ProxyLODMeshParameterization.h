// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProxyLODMeshTypes.h"  // FVertexDataMesh
#include "ProxyLODGrid2d.h" // FTextureAtlasDesc

#include <functional> // Note: should convert std::function to TFunction

/**
*  In mesh segmentation (charts and atlases) and parameterization (generating UVs ) we rely 
*  primarily on the open source DirectX Mesh code.  The methods here provide access to that 
*  functionality.
* 
*/

namespace ProxyLOD
{
	/**
	* Primary entry point:
	* Method that generates new UVs on the FVertexDataMesh according to the parameters specified in the FTextureAtlasDesc
	* The underlying code uses Isometeric approach (Iso-Charts) in UV generation.
	*
	* As a debugging option, the updated InOutMesh can have vertex colors that distinguish the various UV charts.
	*
	* NB: The mesh vertex count may change as vertices are split on UV seams.
	* 
	* @param InOutMesh            Mesh that will be updated by this function, adding UVs
	* @param InTextureAtlasDesc   Description of the texel resolution of the desired texture atlas.
	* @param bVertColorParts      Option to add vertex colors according the the chart in the texture atlas, used for debugging.
	*
	* @return 'true' if the UV generation succeeded,  'false' if it failed.
	*/
	bool GenerateUVs( FVertexDataMesh& InOutMesh, const FTextureAtlasDesc& InTextureAtlasDesc, const bool bVertexColorParts = false);



	/**
	* Lower-level entry point:
	* Method that generates new UVs on the FVertexDataMesh according to the parameters specified in the FTextureAtlasDesc
	* The underlying code uses Isometeric approach (Iso-Charts) in UV generation.
	*
	* As a debugging option, the updated InOutMesh can have vertex colors that distinguish the various UV charts.
	*
	* NB: The mesh vertex count may change as vertices are split on UV seams.
	*
	* @param InOutMesh            Mesh that will be updated by this function, adding UVs
	* @param InTextureAtlasDesc   Description of the texel resolution of the desired texture atlas.
	* @param bVertColorParts      Option to add vertex colors according the the chart in the texture atlas, used for debugging.
	* @param MaxStretch           The maximum amount of stretch between (0 - none and 1 - any)
	* @param MaxChartNumber       Maximum number of charts required for the atlas.  If this is 0, will be parameterized solely on stretch. 
	*                             Note, not a hard limit - isochart will stop when a valid charting is found that is greater or equal to this number.
	* @param bComputeIMTNormal    Compute a metric tensor from the normals.  Generally not a good thing.
	* @param StatusCallBack       Allows for termination of the UV generation.
	* @param MaxStretchOut        Actual max stretch used in computing uvs
	* @param NumChartsOut         Number of charts actually generated
	*
	* @return 'true' if the UV generation succeeded,  'false' if it failed.
	*/
	bool GenerateUVs(FVertexDataMesh& InOutMesh, 
		             const FTextureAtlasDesc& TextureAtlasDesc, 
		             const bool bVertexColorParts,
				     const float MaxStretch, 
		             const size_t MaxChartNumber, 
		             const bool bComputeIMTNormal,
		             std::function<HRESULT __cdecl(float percentComplete)> StatusCallBack,
		             float* MaxStretchOut = NULL, 
		             size_t* NumChartsOut = NULL);


	/**
	* Lower-level entry point:
	* Method that generates new UVs inconstant with the mesh defined by the VertexBuffer/IndexBuffer 
	* according to the parameters specified in the FTextureAtlasDesc
	* The underlying code uses Isometeric approach (Iso-Charts) in UV generation.
	* This assumes that the caller has already generated the mesh adjacency information. 
	*
	*
	*
	* @param InTextureAtlasDesc   Description of the texel resolution of the desired texture atlas.
	* @param VertexBuffer         Positions of vertices
	* @param IndexBuffer          Triangle definitions
	* @param AdjacencyBuffer      3 entries for each face, each entry is the face index of the face that is adjacent to the given face. -1 means no face.
	* @param StatusCallBack       Allows for termination of the UV generation.
	* @param UVVertexBuffer       Resulting UVs
	* @param UVIndexBuffer        Connectivity for the UVs
	* @param VertexRemapArray     Array that remaps the vertices split by the UV generation to the source vertices.
	* @param MaxStretch           On Input, The maximum amount of stretch between (0 - none and 1 - any), on Output the actual max stretch used.
	* @param MaxChartNumber       On Input Maximum number of charts required for the atlas.  If this is 0, will be parameterized solely on stretch.
	*                             Note, not a hard limit - isochart will stop when a valid charting is found that is greater or equal to this number.
	*                             On Output: Number of charts actually generated
	*
	* @return 'true' if the UV generation succeeded,  'false' if it failed.
	*/
	bool GenerateUVs(const FTextureAtlasDesc& TextureAtlasDesc,
		             const TArray<FVector3f>&   VertexBuffer,
		             const TArray<int32>&     IndexBuffer,
		             const TArray<int32>&     AdjacencyBuffer,
		             TFunction<bool(float)>&  Callback,
		             TArray<FVector2D>&       UVVertexBuffer,
		             TArray<int32>&           UVIndexBuffer,
		             TArray<int32>&           VertexRemapArray,
		             float&                   MaxStretch,
		             int32&                   NumCharts);

	/**
	* Generate adjacency data needed for the mesh, additionally this may alter the mesh in attempting to 
	* remove mesh degeneracy problems.  This method is primarily called within GenerateUVs
	*
	* @param InOutMesh          Mesh to process.
	* @param OutAdjacency       Adjacency data understood by the DirectX Mesh code.
	*
	* @return  'true' if the mesh was successfully cleaned of all bow-ties.
	*/
	bool GenerateAdjacenyAndCleanMesh(FVertexDataMesh& InOutMesh, std::vector<uint32>& OutAdjacency);
 
	/**
	* Generate mesh adjacency used by mesh clean code and uv generation code.
	* Various mesh types are supported.
	*
	* @param InOutMesh          Mesh to process.
	* @param OutAdjacency       Adjacency data understood by the DirectX Mesh code.
	*
	*/
	void GenerateAdjacency(const FVertexDataMesh& InMesh, std::vector<uint32>& OutAdjacencyArray);
	void GenerateAdjacency(const FAOSMesh& InMesh, std::vector<uint32>& OutAdjacencyArray);
	void GenerateAdjacency(const FMeshDescription& InMesh, std::vector<uint32>& OutAdjacencyArray);
}