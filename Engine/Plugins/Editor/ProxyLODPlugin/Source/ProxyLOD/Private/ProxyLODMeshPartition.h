// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProxyLODMeshTypes.h"
#include "ProxyLODThreadedWrappers.h"

// used by SpatialMeshSplit
#include <vector>
#include <unordered_map>

// used by mesh merge
#include "Containers/HashTable.h"

/**
*  Methods used in partitioning a mesh into multiple meshes and reconnecting partitions into a single mesh.
*  Designed for use in parallel simplification ( simplifier instances run on each mesh )
* 
*
*  The partitioning code uses a functor based partition assignment and is general purpose, but the
*  code to reconnect partitions relies on external information in the form of idenified seams for the 
*  reconnection.
*/
namespace ProxyLOD
{

	/** 
	* Create an array that maps triangle faces to various partitions.
	*
	* Required Interface
	*    struct FunctorInterface
	*    {
	*        int32 operator()(const FVector (&In)[3]);
	*        int32 NumPartitions() const ;
	*     };
	* 
	* @param InMesh             Triangle mesh - the various faces will be assigned to partitions.
	* @param Functor            The partition functor takes the three verts of a triangle and returns a partition value
	*                          
	* @param OutPartitionArray  Has one entry for each face in the mesh - OutPartitionArray[i] indicates the partition assignment 
	*                           for face 'i'.  Any previous content will be destroyed.
	*/
	template <typename VertexType, typename PartitionFunctor>
	void CreatePartitionArray( const TAOSMesh<VertexType>& InMesh,
		                       const PartitionFunctor& Functor,
		                       TArray<int32>& OutPartitionArray);

	/**
	* Generate an array of meshes that results from applying the partition functor to spatially split the mesh.
	* 
	* Required Interface
	*    struct FunctorInterface
	*    {
	*        int32 operator()(const FVector (&In)[3]);
	*        int32 NumPartitions() const ;
	*     };
	*
	* @param InMesh        Mesh to partition into multiple meshes.
	* @param Functor       A functor that controls face to partition assignment.
	* @param OutMeshArray  Array of meshes that corresponds to the original geometry.  Any previous content will be destroyed.
	*
	*/
	template <typename VertexType, typename PartitionFunctor>
	void PartitionMeshSplit( const TAOSMesh<VertexType>& InMesh,
	              	         const PartitionFunctor& Functor,
		                     TArray<TAOSMesh<VertexType>>& OutMeshArray);

	/**
	* Decompose the bbox into "NumPartitions" equal sized blocks along the major axis, numbered [0.. NumPartitions-1].
	* A triangle is assigned to the partition with smallest id number that contains one of its verts.
	* On return ResultMeshArray will hold a unique mesh for each logical partition.
	*
	* @param InMesh        Mesh to partition into multiple meshes.
	* @param InBBox        The bounds that are subdivided into spatial partitions.
	* @param OutMeshArray  Array of meshes that corresponds to the original geometry.  Any previous content will be destroyed.
	*
	*/
	template <typename VertexType>
	static void PartitionOnMajorAxis( const TAOSMesh<VertexType>& InMesh,
		                              const ProxyLOD::FBBox& InBBox,
		                              uint32 InNumPartitions,
		                              TArray<TAOSMesh<VertexType>>& OutMeshArray);

	/**
	* Merges the input MeshArray into a single mesh.
	*
	* NB: Note this destroys the input mesh array 
	* NB: the resulting mesh has split verts on the seams of the merge.
	*
	* @param InMeshArray  Array of meshes.  Will be destroyed.
	* @param OutMesh      Container holding the union of all meshes.  Will have split verts on seams.
	*
	*/
	template <typename VertexType>
	bool  MergeMeshArray( TArray<TAOSMesh<VertexType>>& InMeshArray,
		                  TAOSMesh<VertexType>& OutMesh);
	/**
	* Merges the input MeshArray into a single mesh.  This function makes use of tagged vertexes, called seam verts 
	* that have been identified as having a duplicate vertex in at least one other mesh.  In use these seam verts are
	* identified by the simplifier as "locked" verts.
	*
	* NB: the duplicate locked verts should be eliminated.
	*
	* @param InMeshArray  Array of meshes.  Will be destroyed
	* @param InSeamVerts  Array of Arrays of seam vertex identifiers.  One array of seam vertexes for each mesh
	*                     in the mesh array.  These are special tags that indicate a duplicate (also taged) vertex should exist
	*                     in at least one of the other meshes.
	* 
	* @param OutMesh      On return this should hold a fully merged mesh - all "seam" vertices should be merged.     
	*/
	template <typename VertexType>
	bool  MergeMeshArray( TArray<TAOSMesh<VertexType>>& InMeshArray,
		                  const TArray<int32> * InSeamVertArrayOfArrays,
		                  TAOSMesh<VertexType>& OutMesh);


}

// --- Implementation of templated functions --- 

	template <typename VertexType, typename PartitionFunctor>
	void ProxyLOD::CreatePartitionArray( const TAOSMesh<VertexType>& Mesh, 
		                                        const PartitionFunctor& Functor, 
		                                        TArray<int32>& PartitionArray)
	{
		// The number of triangles

		const int32 NumTris = Mesh.GetNumIndexes() / 3;

		// Allocate space for the results.

		ResizeArray(PartitionArray, NumTris);

		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumTris), [&Mesh, &Functor, &PartitionArray](const ProxyLOD::FIntRange& Range)
		{
			const VertexType* VertArray = Mesh.Vertexes;
			const uint32*     IndexArray = Mesh.Indexes;

			// loop over the tris

			for (int32 tri = Range.begin(), Tri = Range.end(); tri < Tri; ++tri)
			{
				int32 Offset = tri * 3;
				uint32 Idxs[3] = { IndexArray[Offset], IndexArray[Offset + 1], IndexArray[Offset + 2] };

				FVector Verts[3] = { (FVector)VertArray[Idxs[0]].GetPos(), (FVector)VertArray[Idxs[1]].GetPos(), (FVector)VertArray[Idxs[2]].GetPos() };

				PartitionArray[tri] = Functor(Verts);
			}
		});

	}


	template <typename VertexType, typename PartitionFunctor>
	void ProxyLOD::PartitionMeshSplit( const TAOSMesh<VertexType>& SrcMesh, 
		                                      const PartitionFunctor& Functor, 
		                                      TArray<TAOSMesh<VertexType>>& ResultMeshArray)
	{
		typedef TAOSMesh<VertexType>   MeshType;

		const int32 NumPartitions = Functor.NumPartitions();

		// Array that will hold the partition index for each triangle in the source mesh

		TArray<int32> PartitionArray;

		// Create a partition of the triangles in the SrcMesh

		CreatePartitionArray(SrcMesh, Functor, PartitionArray);

		// Establish space for the different partition results

		ResultMeshArray.Empty(NumPartitions);
		// Add empty meshes.  NB: this only works because the default TOSMesh constructor is so simple  
		ResultMeshArray.AddZeroed(NumPartitions);



		// Fill each partition mesh (going wide over partitions)

		ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumPartitions), [&SrcMesh, &PartitionArray, &ResultMeshArray](const ProxyLOD::FIntRange& Range)
		{
			// loop over the partitions.
			for (int32 p = Range.begin(), P = Range.end(); p < P; ++p)
			{

				std::vector<VertexType>  TmpVertArray;
				std::vector<uint32>      TmpIdxArray;
				std::unordered_map<uint32, uint32> IndxMap;  // maps index in old vert array to index in local vert array.

				for (int32 i = 0, I = PartitionArray.Num(); i < I; ++i)
				{
					// if triangle belongs to this partition, 
					// add it to the TmpIdxArray and add any new verts to the TmpVertArray 
					if (PartitionArray[i] == p)
					{
						// the three verts
						for (int32 v = i * 3; v < (i + 1) * 3; ++v)
						{
							const auto VertIdx = SrcMesh.Indexes[v];
							auto LocalIdxItr = IndxMap.find(VertIdx);

							// we already have this vertex
							if (LocalIdxItr != IndxMap.end())
							{
								TmpIdxArray.push_back(LocalIdxItr->second);
							}
							else
							{
								// add this vertex to our vertex array
								TmpVertArray.push_back(SrcMesh.Vertexes[VertIdx]);
								uint32 LocalIdx = TmpVertArray.size() - 1;
								TmpIdxArray.push_back(LocalIdx);
								IndxMap[VertIdx] = LocalIdx;
							}
						}
					}
				}

				// Copy the results into the Mesh
				MeshType& Mesh = ResultMeshArray[p];
				Mesh.Resize(TmpVertArray.size(), TmpIdxArray.size() / 3);

				for (int32 i = 0; i < TmpVertArray.size(); ++i) Mesh.Vertexes[i] = TmpVertArray[i];
				for (int32 i = 0; i < TmpIdxArray.size(); ++i)  Mesh.Indexes[i] = TmpIdxArray[i];

			}

		});

	}

	namespace ProxyLOD
	{
		class FMajorAxisPartitionFunctor
		{
		public:
			FMajorAxisPartitionFunctor(const ProxyLOD::FBBox& BBox, const int32 NumPartitions)
				: Num(NumPartitions)
			{
				// Determine the major axis: 0, 1, or 2
				MajorIndex = BBox.maxExtent();

				Min = BBox.min()[MajorIndex];

				NumOnLength = float(NumPartitions) / (BBox.max()[MajorIndex] - Min);

			}
			FMajorAxisPartitionFunctor(const FMajorAxisPartitionFunctor& other) :
				Num(other.Num), MajorIndex(other.MajorIndex), Min(other.Min), NumOnLength(other.NumOnLength)
			{}

			int32 operator()(const FVector(&Verts)[3]) const
			{
				// classify the triangle based on the min projection of the vert on the major axis
				float SmallestVert = FMath::Min3(Verts[0][MajorIndex], Verts[1][MajorIndex], Verts[2][MajorIndex]);
				const int32 PartitionValue = FMath::FloorToInt(NumOnLength * (SmallestVert - Min));

				return FMath::Clamp(PartitionValue, 0, Num - 1);
			};

			int32 NumPartitions() const { return Num; }

		private:
			int32 Num;
			int32 MajorIndex;
			float Min;
			float NumOnLength;
		};
	}

	
	template <typename VertexType>
	static void ProxyLOD::PartitionOnMajorAxis( const TAOSMesh<VertexType>& SrcMesh, 
		                                        const ProxyLOD::FBBox& BBox, 
		                                        uint32 NumPartitions, 
		                                        TArray<TAOSMesh<VertexType>>& ResultMeshArray)
	{

		// Create a partition functor

		FMajorAxisPartitionFunctor PartitionFunctor(BBox, NumPartitions);

		// Partition the mesh 

		PartitionMeshSplit(SrcMesh, PartitionFunctor, ResultMeshArray);

	}

	
	template <typename VertexType>
	bool  ProxyLOD::MergeMeshArray( TArray<TAOSMesh<VertexType>>& MeshArray, TAOSMesh<VertexType>& ResultMesh)
	{
		// Determine the space needed.

		const int32 NumMeshes = MeshArray.Num();
		TArray<uint32> VrtOffsets;
		TArray<uint32> IdxOffsets;

		ResizeArray(VrtOffsets, NumMeshes + 1);
		ResizeArray(IdxOffsets, NumMeshes + 1);

		VrtOffsets[0] = 0;  IdxOffsets[0] = 0;

		for (int32 i = 0; i < NumMeshes; ++i)
		{
			const auto& Mesh = MeshArray[i];

			VrtOffsets[i + 1] = VrtOffsets[i] + Mesh.GetNumVertexes();
			IdxOffsets[i + 1] = IdxOffsets[i] + Mesh.GetNumIndexes();
		}

		// All the meshes were empty
		if (VrtOffsets[NumMeshes] == 0) return false;

		ResultMesh.Resize(VrtOffsets[NumMeshes] /*vert count*/, IdxOffsets[NumMeshes] / 3 /*face count*/);

		for (int32 m = 0; m < NumMeshes; ++m)
		{
			auto& Mesh = MeshArray[m];

			// copy verts.  This should be a memcopy.
			uint32 VOffset = VrtOffsets[m];
			for (uint32 v = 0; v < Mesh.GetNumVertexes(); ++v)
			{
				ResultMesh.Vertexes[VOffset + v] = Mesh.Vertexes[v];
			}

			// copy indexes, and update
			uint32 IOffset = IdxOffsets[m];
			for (uint32 i = 0; i < Mesh.GetNumIndexes(); ++i)
			{
				ResultMesh.Indexes[IOffset + i] = VOffset + Mesh.Indexes[i];
			}

			// destroy this mesh
			Mesh.Empty();
		}

		return true;
	}



	template <typename VertexType>
	bool  ProxyLOD::MergeMeshArray( TArray<TAOSMesh<VertexType>>& MeshArray, 
		                                   const TArray<int32> * SeamVertArrayOfArrays, 
		                                   TAOSMesh<VertexType>& ResultMesh)
	{

		TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::MergeMeshArray)
		const int32 NumPartitions = MeshArray.Num();


		ResultMesh.Empty();

		// The total number of locked verts

		int32 NumSeamVerts = 0;
		for (int32 i = 0; i < NumPartitions; ++i)
		{
			NumSeamVerts += SeamVertArrayOfArrays[i].Num();
		}

		// If there were no locked verts, then we can do a simple merge.

		if (NumSeamVerts == 0)
		{
			MergeMeshArray(MeshArray, ResultMesh);
			return true;
		}

		// Total number of mesh sections to merge.

		const int32 MeshSectionCount = MeshArray.Num();

		// Create a single list of all the locked verts.
		// To index the verts we used a global index that
		// equivalent to copying all verts in the MeshArray
		// into a single array.

		TArray<uint32> SeamVerts;
		ResizeArray(SeamVerts, NumSeamVerts);
		{
			int32 VertOffset = 0;
			int32 DstOffset = 0;
			for (int32 i = 0; i < MeshSectionCount; ++i)
			{
				const TArray<int32>& SeamVertArray = SeamVertArrayOfArrays[i];
				const int32 NumLocalSeamVerts = SeamVertArray.Num();

				for (int32 j = 0; j < NumLocalSeamVerts; ++j)
				{
					SeamVerts[j + DstOffset] = VertOffset + SeamVertArray[j];
				}

				VertOffset += MeshArray[i].GetNumVertexes();
				DstOffset += NumLocalSeamVerts;
			}
		}


		TAOSMesh<VertexType> TmpMesh;

		// Merge the meshes into a temp mesh.  This will have duplicated verts.
		// NB: this destroys/empties the meshes in the mesh array.

		MergeMeshArray(MeshArray, TmpMesh);

		// Lets verify that each lock vert has a twin.

		FHashTable PositionHashTable(1024, NumSeamVerts);

		auto HashPoint = [](const FVector& Pos) ->uint32
		{
			union { float f; uint32 i; } x;
			union { float f; uint32 i; } y;
			union { float f; uint32 i; } z;

			x.f = Pos.X;
			y.f = Pos.Y;
			z.f = Pos.Z;

			return Murmur32({ x.i, y.i, z.i });
		};


		// Hash the locations of each duplicate mesh vertex

		TArray<uint32> PositionHashValues;
		ResizeArray(PositionHashValues, NumSeamVerts);
		for (int32 i = 0; i < NumSeamVerts; ++i)
		{
			// Vert Id
			const uint32 VertId = SeamVerts[i];
			const FVector& Position = (FVector)TmpMesh.Vertexes[VertId].Position;
			const uint32 HashValue = HashPoint(Position);
			PositionHashValues[i] = HashValue;
			PositionHashTable.Add(HashValue, VertId);

		}


		// Create a mapping between Locked Verts.
		// LockedVert[i] = VertId,  RegularVerts[i] = RegularVert
		// That is the Vert at VertId is a duplicate of the RegularVert
		// that we should use instead to weld the mesh sections together.

		TArray<uint32> RegularVertIds;
		ResizeArray(RegularVertIds, NumSeamVerts);

		for (int32 i = 0; i < NumSeamVerts; ++i)
		{
			const uint32 VertId = SeamVerts[i];
			const FVector& Position = (FVector)TmpMesh.Vertexes[VertId].Position;
			const int32 HashValue = PositionHashValues[i];

			// Collect the indexes of all the verts that share location with this one.
			TArray<uint32, TInlineAllocator<4> > Duplicates;
			for (uint32 j = PositionHashTable.First(HashValue); PositionHashTable.IsValid(j); j = PositionHashTable.Next(j))
			{
				if (j != VertId) // make sure we aren't comparing with ourself.
				{
					const FVector& OtherPosition = (FVector)TmpMesh.Vertexes[j].Position;

					// Are the positions the same?
					if (Position == OtherPosition)
					{
						Duplicates.Push(j);
					}
				}
			}



			// We will just use the vert with the smallest global index as the final "true" vertex
			uint32 RemapValue = VertId;
			for (int32 j = 0; j < Duplicates.Num(); ++j)
			{
				RemapValue = FMath::Min(RemapValue, Duplicates[j]);
			}

			RegularVertIds[i] = RemapValue;
		}


		{
			TArray<uint32> VertMap;
			ResizeArray(VertMap, TmpMesh.GetNumVertexes());

			for (int32 i = 0; i < VertMap.Num(); ++i) VertMap[i] = (uint32)i;

			for (int32 i = 0; i < NumSeamVerts; ++i)
			{
				uint32 OldVertIdx = SeamVerts[i];
				uint32 RegularIdx = RegularVertIds[i];

				VertMap[OldVertIdx] = RegularIdx;
			}

			// Update the index array in the temp mesh to use the regular verts.
			// This means for every locked vert pair, one vert is now ignored.

			for (uint32 i = 0; i < TmpMesh.GetNumIndexes(); ++i)
			{
				TmpMesh.Indexes[i] = VertMap[TmpMesh.Indexes[i]];
			}
		}



		TArray<uint32> TmpIndexArray;
		ResizeArray(TmpIndexArray, TmpMesh.GetNumIndexes());

		std::vector<VertexType> TmpVertArray;
		{
			std::unordered_map<uint32, uint32> VertexCompactMap;
			// Copy the index buffer into the result mesh while compacting the vertex array.
			for (uint32 Idx = 0; Idx < TmpMesh.GetNumIndexes(); ++Idx)
			{
				const uint32 OldVertId = TmpMesh.Indexes[Idx];

				auto VertIdItr = VertexCompactMap.find(OldVertId);

				// we already have this vertex
				if (VertIdItr != VertexCompactMap.end())
				{
					TmpIndexArray[Idx] = VertIdItr->second;
				}
				else
				{
					// add this vertex to our vertex array
					TmpVertArray.push_back(TmpMesh.Vertexes[OldVertId]);
					uint32 VertId = TmpVertArray.size() - 1;
					TmpIndexArray[Idx] = VertId;
					VertexCompactMap[OldVertId] = VertId;
				}
			}
		}

		ResultMesh.Resize(TmpVertArray.size(), TmpIndexArray.Num() / 3);

		// Copy the compacted vert array into the result mesh

		for (uint32 i = 0; i < ResultMesh.GetNumVertexes(); ++i)
		{
			ResultMesh.Vertexes[i] = TmpVertArray[i];
		}
		// Copy the index array into the result mesh.

		for (uint32 i = 0; i < ResultMesh.GetNumIndexes(); ++i)
		{
			ResultMesh.Indexes[i] = TmpIndexArray[i];
		}

		return true;
	}
