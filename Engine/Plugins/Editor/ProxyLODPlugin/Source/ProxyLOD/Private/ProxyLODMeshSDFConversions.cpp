// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODMeshSDFConversions.h"
#include "ProxyLODMeshConvertUtils.h"
#include "ProxyLODOpenVDB.h"

#include "CoreMinimal.h"


/**
* Generate a new SDF (with narrow band thickness of 2) that represents moving the zero crossing
* the specified distance in either the positive or negative normal direction.
*
* NB: This will fail if the offset is greater than 2 voxels.

*
* @param  InSDFVolume            SDF grid with assumed narrow band of 2
* @param  WSOffset               World Space Distance to offset the zero.  This should be in the range -2dx : 2dx. 
*                                where dx is the input grid voxel size
* @param  ResultVoxelSize        The voxel size used in the resulting grid.
*
* @return A new SDF that represents a dilation or erosion (expansion or contraction) of the original SDF
*/
static openvdb::FloatGrid::Ptr OffsetSDF(const openvdb::FloatGrid::Ptr InSDFVolume, const double WSOffset, const double ResultVoxelSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(OffsetSDF)
	// Extract the iso-surface with offset DilationInVoxels.

	// The voxel size in world space units : taking the first element is okay, since the voxels are square.

	const double VoxelSize = InSDFVolume->transform().voxelSize()[0];
	
	// check that the offset is contained in the narrow band of 2 voxels on each side.
	checkSlow(2. * VoxelSize > WSOffset && 2. * VoxelSize > -WSOffset);

	const double IsoValue = WSOffset;

	FMixedPolyMesh  MixedPolyMesh;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OpenVDB::VolumeToMesh)
		openvdb::tools::volumeToMesh(*InSDFVolume, MixedPolyMesh.Points, MixedPolyMesh.Triangles, MixedPolyMesh.Quads, IsoValue, 0.001);
	}

	// Create a new empty grid with the same transform and metadata
	openvdb::FloatGrid::Ptr OutSDFVolume = openvdb::FloatGrid::create(*InSDFVolume);
	OutSDFVolume->setTransform(openvdb::math::Transform::createLinearTransform(ResultVoxelSize));

	// Re-voxelize with bandwidth 2
	MixedPolyMesh.Transform = OutSDFVolume->transform();
	ProxyLOD::MeshToSDFVolume(MixedPolyMesh, OutSDFVolume->transform(), OutSDFVolume);

	return OutSDFVolume;
}


void ProxyLOD::CloseGaps(openvdb::FloatGrid::Ptr InOutSDFVolume, const double GapRadius, const int32 MaxDilations)
{
	// Implementation notes:
	// This functions by first inflating (dilate) the geometry SDF (moving the surface outward along the normal) an amount 
	// GapRadius.  Doing this may bring surfaces into contact, thus closing gaps.
	// Next the geometry SDF with merged gaps is deflated (erode) to a size that should be slightly smaller than the original geometry.
	// Lastly a union between the deflated, gap-merged geometry and a copy of the original SDF is formed.
	// NB: this relies on the fact that grid-based discretization of the SDF at each step of dilation and erosion also smooths
	// the SDF (dilation isn't exactly reversed by erosion).


	// Early out for invalid input.

	if (!InOutSDFVolume)
	{
		return;
	}

	// The voxel size for this grid

	const double InputVoxelSize = InOutSDFVolume->transform().voxelSize()[0];

	// If the gap radius is too small, this won't have an effect.

	if (GapRadius < InputVoxelSize)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::CloseGaps)
	const double MaxOffsetInVoxels = 1.5;

	// Step configuration using InputVoxelSize

	const double DefaultStepSize  = MaxOffsetInVoxels * InputVoxelSize;
	const int32  DefaultStepNum   = FMath::FloorToInt(float( GapRadius / (MaxOffsetInVoxels * InputVoxelSize)));
	const double DefaultRemainder = GapRadius - DefaultStepNum * DefaultStepSize;

	// Alternate step configuration, deduce working voxel size from MaxIterations

	const double AltStepSize  = (GapRadius - InputVoxelSize) / MaxDilations;
	const int32  AltStepNum   = MaxDilations;
	const double AltRemainder = InputVoxelSize;

	const bool bUseDefaultValues = !(MaxDilations < DefaultStepNum);

	// Choose the correct values to use.  Either dilate and erode with the default voxelsize, or using a bigger voxel size.

	double WorkingStepSize;
	double WorkingRemainder;
	double WorkingVoxelSize;
	int32 StepNum;
	if (bUseDefaultValues)
	{
		WorkingStepSize  = DefaultStepSize;
		WorkingRemainder = DefaultRemainder;
		WorkingVoxelSize = InputVoxelSize;
		StepNum          = DefaultStepNum;
	}
	else
	{
		WorkingStepSize  = AltStepSize;
		WorkingRemainder = AltRemainder;
		WorkingVoxelSize = AltStepSize / MaxOffsetInVoxels;
		StepNum          = AltStepNum;
	}

	openvdb::FloatGrid::Ptr TmpGrid = InOutSDFVolume;

	const bool bRequireRemainder = (!bUseDefaultValues || WorkingRemainder > 0.1 * InputVoxelSize);

	// -- Dilate

	if (bRequireRemainder)
	{    
		// Note: from inputVoxelSize to WorkingVoxelSize
		TmpGrid = OffsetSDF(TmpGrid, WorkingRemainder, WorkingVoxelSize); 
	}

	for (int32 step = 0; step < StepNum; ++step)
	{
		TmpGrid = OffsetSDF(TmpGrid, WorkingStepSize, WorkingVoxelSize);
	}

	// -- Erode

	for (int32 step = 0; step < StepNum; ++step)
	{
		TmpGrid = OffsetSDF(TmpGrid, -WorkingStepSize, WorkingVoxelSize);
	}

	if (bRequireRemainder)
	{
		// Note: from WorkingVoxelSize to InputVoxelSize
		TmpGrid = OffsetSDF(TmpGrid, -WorkingRemainder, InputVoxelSize);
	}

	// Additional Erode to shrink a little more so this hole-filled surface is slightly offset from the higher-quality
	// original surface

	TmpGrid = OffsetSDF(TmpGrid, -.5 * InputVoxelSize, InputVoxelSize);
	
	// Union with the higher quality source (this will add the hole plugs..) 
	
	openvdb::tools::csgUnion(*InOutSDFVolume, *TmpGrid);

	// reduce memory footprint, increase sparseness

	const float HalfBandWidth = 2.f;
	openvdb::tools::pruneLevelSet(InOutSDFVolume->tree(), HalfBandWidth, -HalfBandWidth);

}

void ProxyLOD::RemoveClipped(openvdb::FloatGrid::Ptr InOutSDFVolume, openvdb::FloatGrid::Ptr ClippingVolume)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::RemoveClipped)

	// do a difference that deletes the clippling volume from the geometry.

	openvdb::tools::csgDifference(*InOutSDFVolume, *ClippingVolume, true);

	// reduce memory footprint, increase sparseness

	const float HalfBandWidth = 2.f;
	openvdb::tools::pruneLevelSet(InOutSDFVolume->tree(), HalfBandWidth, -HalfBandWidth);
}
