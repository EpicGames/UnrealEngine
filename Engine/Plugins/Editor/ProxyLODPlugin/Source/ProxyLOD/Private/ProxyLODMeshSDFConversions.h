// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ProxyLODMeshTypes.h"
#include "ProxyLODMeshConvertUtils.h"
#include "ProxyLODMeshUtilities.h"
#include "ProxyLODOpenVDB.h"

namespace ProxyLOD
{

	/**
	* Generate a single sparse Signed Distance Field (SDF) Grid from the polys managed by InMesh,  
	* Optionally store the index of the closest poly is recorded in a sparse PolyIndexGrid.  
	*
	* @param  InMesh			Source polygons to be voxelized.
	* @param  OutSDFGrid        Resulting sparse signed distance field.
	* @param  OutPolyIndexGrid  Optional resulting sparse grid. For each voxel with valid distance in the OutSDFGrid
	*                           the index of the closest poly will be recorded in the corresponding voxel 
	*                           of the OutPolyIndedGrid.
	*
	* @retun 'true' is success, 'false' will generally be an out of memory error.
	*/
	template <typename SrcMeshType>
	bool MeshToSDFVolume( const SrcMeshType& InMesh,
						  const openvdb::math::Transform& InTransform,
	                      openvdb::FloatGrid::Ptr& OutSDFGrid,
		                  openvdb::Int32Grid* OutPolyIndexGrid = nullptr);

	/**
	* Extract the isosurface (=IsoValue) of the given SDF volume, in the form of triangle mesh.
	*
	* NB: This produces geometry only, and ignores Tangents, Texture Coordinates and Colors etc. 
	* 
	* @param  InSDFVolume       Sparse signed distance field: voxel based representation of geometry.  
	* @param  IsoValue          Indicates the surface defined by this distance value.
	* @param  AdaptivityValue   Internal adaptivity attempts to simplify regions of low curvature. 
	*                           Local face normals are compared using this value as a threshold.
	* @param  OutMesh           Resulting triangular mesh that is consistent with the surface defined by the iso-value.   
	*/
	template <typename DstMeshType>
	void SDFVolumeToMesh( const openvdb::FloatGrid::ConstPtr InSDFVolume, 
		                  const double IsoValue, 
						  const double AdaptivityValue, 
		                  DstMeshType& OutMesh);

	/**
	* Extract the isosurface (=IsoValue) of the given SDF volume, in the form of triangle mesh and adds
	* very simple vertex normals.
	*
	* NB: This produces geometry only, and ignores Tangents, Texture Coordinates and Colors etc.
	*
	* @param  InSDFVolume       Sparse signed distance field: voxel based representation of geometry.
	* @param  IsoValue          Indicates the surface defined by this distance value.
	* @param  AdaptivityValue   Internal adaptivity attempts to simplify regions of low curvature.
	*                           Local face normals are compared using this value as a threshold.
	* @param  OutMesh           Resulting triangular mesh that is consistent with the surface defined by the iso-value.
	*/
	template <typename AOSVertexType>
	void ExtractIsosurfaceWithNormals( const openvdb::FloatGrid::ConstPtr InSDFVolume, 
		                              const double IsoValue, 
		                              const double Adaptivity, 
		                              TAOSMesh<AOSVertexType>& OutMesh);

	/**
	* Updates the InOutSDFVollume to potentially close small holes and gaps in the iso-surface.
	*
	* This is particularly useful when processing distant buildings (where the doors
	* and windows may only be a few voxels in size).  In this case, closing the openings
	* may result in a water-tight mesh and thus remove interior geometry.
	* 
	* NB: In addition to closing holes, this may merge nearby separate objects.
	*
	* @param InOutSDFVolume      Signed distance field that will be altered
	* @param GapRadius           1/2 the length scale of the gap or hole size.
	* @param MaxDilations        The max allowed number of dilation/erosion pairs,
	*
	* NB: If the GapRadius is large compared with the voxelsize of the input volume, 
	* the dilations and erosions will be done with a larger voxelsize (as determined
	* by MaxDilations).
	*                            
	*/
	void CloseGaps(openvdb::FloatGrid::Ptr InOutSDFVolume, const double GapRadius, const int32 MaxDilations);

	/**
	* Update the InOutSDFVolume by doing a CSG difference with the Clipping volume.
	*
	* NB: In addition to altering the InOutSDFVolume, this may destroy the clipping volume
	*
	* @param InOutSDFVolume     Signed distance field representing the interesting geometry.
	* @param ClippingVolume     Signed distance field representing the region to be subtracted from the interesting geometry.
	*
	*/
	void RemoveClipped(openvdb::FloatGrid::Ptr InOutSDFVolume, openvdb::FloatGrid::Ptr ClippingVolume);

}

template <typename SrcMeshType>
bool ProxyLOD::MeshToSDFVolume(const SrcMeshType& InMesh, const openvdb::math::Transform& InTransform, openvdb::FloatGrid::Ptr& SDFGrid, openvdb::Int32Grid* PolyIndexGrid)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::MeshToSDFVolume)

	bool bSuccess = true;
	try
	{
		const float HalfBandWidth = 2.f;
		int Flags = 0;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(OpenVDB::MeshToVolume)
			SDFGrid = openvdb::tools::meshToVolume<openvdb::FloatGrid>(InMesh, InTransform, HalfBandWidth /*exterior*/, HalfBandWidth/*interior*/, Flags, PolyIndexGrid);
		}

		// reduce memory footprint, increase the sparseness.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(OpenVDB::PruneLevelSet)
			openvdb::tools::pruneLevelSet(SDFGrid->tree(), HalfBandWidth, -HalfBandWidth);
		}
	}
	catch (std::bad_alloc&)
	{
		bSuccess = false;
		if (SDFGrid)
		{
			SDFGrid->tree().clear(); // free any memory held in the smart pointers in the grid
		}
		if (PolyIndexGrid)
		{
			PolyIndexGrid->clear();
		}
	}

	return bSuccess;
}


template <typename DstMeshType>
void ProxyLOD::SDFVolumeToMesh(const openvdb::FloatGrid::ConstPtr SDFVolume, const double IsoValue, const double Adaptivity, DstMeshType& OutMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::SDFVolumeToMesh)

	// we should be generating a new FMeshDescription.

	OutMesh.Empty();

	// Temporary Mesh that supports Quads and Triangles

	FMixedPolyMesh TempMixedPolyMesh;

	// Convert to Quad Mesh - extracting iso-surface defined by IsoValue

	openvdb::tools::volumeToMesh(*SDFVolume, TempMixedPolyMesh.Points, TempMixedPolyMesh.Triangles, TempMixedPolyMesh.Quads, IsoValue, Adaptivity);
	
	MixedPolyMeshToAOSMesh(TempMixedPolyMesh, OutMesh);
}

template <typename AOSVertexType>
void ProxyLOD::ExtractIsosurfaceWithNormals(const openvdb::FloatGrid::ConstPtr SDFVolume, const double IsoValue, const double Adaptivity, TAOSMesh<AOSVertexType>& OutMesh)
{
	SDFVolumeToMesh(SDFVolume, IsoValue, Adaptivity, OutMesh);

	AddNormals(OutMesh);
}

