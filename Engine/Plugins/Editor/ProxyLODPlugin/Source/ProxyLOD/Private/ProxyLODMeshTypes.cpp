// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODMeshTypes.h"
#include "Async/ParallelFor.h"

// --- FMeshDescriptionAdapter ----

FMeshDescriptionAdapter::FMeshDescriptionAdapter(const FMeshDescription& InRawMesh, const openvdb::math::Transform& InTransform) :
	RawMesh(&InRawMesh), Transform(InTransform)
{
	InitializeCacheData();
}

FMeshDescriptionAdapter::FMeshDescriptionAdapter(const FMeshDescriptionAdapter& other)
	: RawMesh(other.RawMesh), Transform(other.Transform)
{
	InitializeCacheData();
}

void FMeshDescriptionAdapter::InitializeCacheData()
{
	VertexPositions = RawMesh->GetVertexPositions();
	TriangleCount = RawMesh->Triangles().Num();

	IndexBuffer.Reserve(TriangleCount * 3);

	for (const FTriangleID TriangleID : RawMesh->Triangles().GetElementIDs())
	{
		IndexBuffer.Add(RawMesh->GetTriangleVertexInstance(TriangleID, 0));
		IndexBuffer.Add(RawMesh->GetTriangleVertexInstance(TriangleID, 1));
		IndexBuffer.Add(RawMesh->GetTriangleVertexInstance(TriangleID, 2));
	}


}

size_t FMeshDescriptionAdapter::polygonCount() const
{
	return size_t(TriangleCount);
}

size_t FMeshDescriptionAdapter::pointCount() const
{
	return size_t(RawMesh->Vertices().Num());
}

void FMeshDescriptionAdapter::getIndexSpacePoint(size_t FaceNumber, size_t CornerNumber, openvdb::Vec3d& pos) const
{
	// Get the vertex position in local space.
	const FVertexInstanceID VertexInstanceID = IndexBuffer[FaceNumber * 3 + CornerNumber];
	// float3 position 
	const FVertexID VertexID = RawMesh->GetVertexInstanceVertex(VertexInstanceID);


	FVector3f Position = VertexPositions[VertexID];
	pos = Transform.worldToIndex(openvdb::Vec3d(Position.X, Position.Y, Position.Z));
};

void FMeshDescriptionArrayAdapter::SetupInstances(int32 MeshCount, TFunctionRef<const FInstancedMeshMergeData* (uint32 Index)> GetMeshFunction)
{
	InstancesTransformArray.Reserve(MeshCount);
	InstancesAdjointTArray.Reserve(MeshCount);

	for (int32 MeshIdx = 0; MeshIdx < MeshCount; ++MeshIdx)
	{
		const FInstancedMeshMergeData* InstancedMeshMergeData = GetMeshFunction(MeshIdx);

		InstancesTransformArray.Add(InstancedMeshMergeData->InstanceTransforms);

		// Cache transpose adjoint matrices
		TArray<FMatrix>& AdjointTArray = InstancesAdjointTArray.Emplace_GetRef();
		for (const FTransform& InstanceTransform : InstancedMeshMergeData->InstanceTransforms)
		{
			FMatrix Matrix = InstanceTransform.ToMatrixWithScale();
			FMatrix AdjointT = Matrix.TransposeAdjoint();
			AdjointT.RemoveScaling();
			AdjointTArray.Add(AdjointT);
		}
	}
}

void FMeshDescriptionArrayAdapter::Construct(int32 MeshCount, TFunctionRef<const FMeshMergeData* (uint32 Index)> GetMeshFunction)
{
	Construct(MeshCount, GetMeshFunction, [](uint32 Index) { return 1; });
}

void FMeshDescriptionArrayAdapter::Construct(int32 MeshCount, TFunctionRef<const FMeshMergeData* (uint32 Index)> GetMeshFunction, TFunctionRef<int32 (uint32 Index)> GetInstanceCountFunction)
{
	PointCount = 0;
	PolyCount = 0;

	PolyOffsetArray.clear();
	IndexBufferArray.clear();
	RawMeshArray.clear();
	MergeDataArray.clear();

	PolyOffsetArray.reserve(MeshCount);
	IndexBufferArray.reserve(MeshCount);
	RawMeshArray.reserve(MeshCount);
	MergeDataArray.reserve(MeshCount);

	PolyOffsetArray.push_back(PolyCount);
	for (int32 MeshIdx = 0; MeshIdx < MeshCount; ++MeshIdx)
	{
		const FMeshMergeData* MergeData = GetMeshFunction(MeshIdx);

		int32 InstanceCount = GetInstanceCountFunction(MeshIdx);
		check(InstanceCount > 0);

		FMeshDescription *RawMesh = MergeData->RawMesh;

		// Sum up all the polys in this mesh.
		int32 MeshPolyCount = RawMesh->Triangles().Num();
		if (MeshPolyCount > 0)
		{
			// Construct local index buffer for this mesh
			IndexBufferArray.push_back(std::vector<FVertexInstanceID>());
			std::vector<FVertexInstanceID>& IndexBuffer = IndexBufferArray.back();
			IndexBuffer.reserve(MeshPolyCount * 3);

			for (const FTriangleID TriangleID : RawMesh->Triangles().GetElementIDs())
			{
				IndexBuffer.push_back(RawMesh->GetTriangleVertexInstance(TriangleID, 0));
				IndexBuffer.push_back(RawMesh->GetTriangleVertexInstance(TriangleID, 1));
				IndexBuffer.push_back(RawMesh->GetTriangleVertexInstance(TriangleID, 2));
			}

			PointCount += size_t(RawMesh->Vertices().Num()) * InstanceCount;
			PolyCount += MeshPolyCount * InstanceCount;

			PolyOffsetArray.push_back(PolyCount);
			RawMeshArray.push_back(RawMesh);
			MergeDataArray.push_back(MergeData);
		}
	}

	RawMeshArrayData.SetNumUninitialized(MergeDataArray.size());

	// The getter constructor is doing expensive operations
	ParallelFor(
		RawMeshArrayData.Num(),
		[this, &GetMeshFunction](int32 MeshIdx)
		{
			const FMeshMergeData* MergeData = MergeDataArray[MeshIdx];
			FMeshDescription *RawMesh = MergeData->RawMesh;
			new (&RawMeshArrayData[MeshIdx]) FMeshDescriptionAttributesGetter(RawMesh);
		},
		EParallelForFlags::Unbalanced
	);

	// Compute the bbox
	ComputeAABB(this->BBox);
}

// --- FMeshDescriptionArrayAdapter ----
FMeshDescriptionArrayAdapter::FMeshDescriptionArrayAdapter(const TArray<const FInstancedMeshMergeData*>& InMergeDataPtrArray)
{
	// Make a default transform.
	Transform = openvdb::math::Transform::createLinearTransform(1.);

	SetupInstances(InMergeDataPtrArray.Num(), [&InMergeDataPtrArray](uint32 Index) { return InMergeDataPtrArray[Index]; });
	Construct(InMergeDataPtrArray.Num(), [&InMergeDataPtrArray](uint32 Index) { return InMergeDataPtrArray[Index]; }, [&InMergeDataPtrArray](uint32 Index) { return InMergeDataPtrArray[Index]->InstanceTransforms.IsEmpty() ? 1 : InMergeDataPtrArray[Index]->InstanceTransforms.Num(); });
}

FMeshDescriptionArrayAdapter::FMeshDescriptionArrayAdapter(const TArray<const FMeshMergeData*>& InMergeDataPtrArray)
{
	// Make a default transform.
	Transform = openvdb::math::Transform::createLinearTransform(1.);

	Construct(InMergeDataPtrArray.Num(), [&InMergeDataPtrArray](uint32 Index) { return InMergeDataPtrArray[Index]; });
}

FMeshDescriptionArrayAdapter::FMeshDescriptionArrayAdapter(const TArray<FMeshMergeData>& InMergeDataArray)
{
	// Make a default transform.
	Transform = openvdb::math::Transform::createLinearTransform(1.);

	Construct(InMergeDataArray.Num(), [&InMergeDataArray](uint32 Index) { return &InMergeDataArray[Index]; });
}

FMeshDescriptionArrayAdapter::FMeshDescriptionArrayAdapter(const TArray<FInstancedMeshMergeData>& InMergeDataArray)
{
	// Make a default transform.
	Transform = openvdb::math::Transform::createLinearTransform(1.);

	SetupInstances(InMergeDataArray.Num(), [&InMergeDataArray](uint32 Index) { return &InMergeDataArray[Index]; });
	Construct(InMergeDataArray.Num(), [&InMergeDataArray](uint32 Index) { return &InMergeDataArray[Index]; }, [&InMergeDataArray](uint32 Index) { return InMergeDataArray[Index].InstanceTransforms.Num(); });
}

FMeshDescriptionArrayAdapter::FMeshDescriptionArrayAdapter(const TArray<FMeshMergeData>& InMergeDataArray, const openvdb::math::Transform::Ptr InTransform)
	: Transform(InTransform)
{
	Construct(InMergeDataArray.Num(), [&InMergeDataArray](uint32 Index) { return &InMergeDataArray[Index]; });
}

FMeshDescriptionArrayAdapter::FMeshDescriptionArrayAdapter(const FMeshDescriptionArrayAdapter& other)
	:Transform(other.Transform), PointCount(other.PointCount), PolyCount(other.PolyCount), BBox(other.BBox)
{
	RawMeshArray = other.RawMeshArray;
	PolyOffsetArray = other.PolyOffsetArray;
	MergeDataArray = other.MergeDataArray;
	InstancesTransformArray = other.InstancesTransformArray;
	InstancesAdjointTArray = other.InstancesAdjointTArray;

	RawMeshArrayData.SetNumUninitialized(RawMeshArray.size());

	// The getter constructor is doing expensive operations
	ParallelFor(
		RawMeshArray.size(),
		[this](int32 MeshIdx)
		{
			new (&RawMeshArrayData[MeshIdx]) FMeshDescriptionAttributesGetter(RawMeshArray[MeshIdx]);
		},
		EParallelForFlags::Unbalanced
	);

	IndexBufferArray.reserve(other.IndexBufferArray.size());
	for (const auto& IndexBuffer : other.IndexBufferArray)
	{
		IndexBufferArray.push_back(IndexBuffer);
	}

}

FMeshDescriptionArrayAdapter::~FMeshDescriptionArrayAdapter()
{
	RawMeshArray.clear();

	RawMeshArrayData.Empty();
	MergeDataArray.clear();
	PolyOffsetArray.clear();
}

void FMeshDescriptionArrayAdapter::GetWorldSpacePoint(size_t FaceNumber, size_t CornerNumber, openvdb::Vec3d& pos) const
{
	int32 MeshIdx, InstanceIdx, LocalFaceNumber;

	const FMeshDescriptionAttributesGetter* AttributesGetter = nullptr;
	const FMeshDescription& RawMesh = GetRawMesh(FaceNumber, MeshIdx, InstanceIdx, LocalFaceNumber, &AttributesGetter);
	check(AttributesGetter);

	const auto& IndexBuffer = IndexBufferArray[MeshIdx];
	// Get the vertex position in local space.
	const FVertexInstanceID VertexInstanceID = IndexBuffer[3 * LocalFaceNumber + int32(CornerNumber)];
	// float3 position 
	FVector3f Position = AttributesGetter->VertexPositions[RawMesh.GetVertexInstanceVertex(VertexInstanceID)];

	if (InstanceIdx != INDEX_NONE)
	{
		Position = (FVector3f)InstancesTransformArray[MeshIdx][InstanceIdx].TransformPosition((FVector)Position);
	}

	pos = openvdb::Vec3d(Position.X, Position.Y, Position.Z);
};

void FMeshDescriptionArrayAdapter::getIndexSpacePoint(size_t FaceNumber, size_t CornerNumber, openvdb::Vec3d& pos) const
{
	openvdb::Vec3d Position;
	GetWorldSpacePoint(FaceNumber, CornerNumber, Position);
	pos = Transform->worldToIndex(Position);

};

const FMeshMergeData& FMeshDescriptionArrayAdapter::GetMeshMergeData(uint32 Idx) const
{
	checkSlow(Idx < MergeDataArray.size());
	return *MergeDataArray[Idx];
}

void FMeshDescriptionArrayAdapter::UpdateMaterialsID()
{
	for (int32 MeshIdx = 0; MeshIdx < MergeDataArray.size(); ++MeshIdx)
	{
		FMeshDescription* MeshDescription = RawMeshArray[MeshIdx];

		check(MergeDataArray[MeshIdx]->RawMesh->Polygons().Num() == MeshDescription->Polygons().Num());
		TMap<FPolygonGroupID, FPolygonGroupID> RemapGroup;
		TArray<int32> UniqueMaterials;
		for (const FPolygonID PolygonID : MeshDescription->Polygons().GetElementIDs())
		{
			FPolygonGroupID NewPolygonGroupID = MergeDataArray[MeshIdx]->RawMesh->GetPolygonPolygonGroup(PolygonID);
			if (!UniqueMaterials.Contains(NewPolygonGroupID.GetValue()))
			{
				UniqueMaterials.Add(NewPolygonGroupID.GetValue());
				FPolygonGroupID OriginalPolygonGroupID = MeshDescription->GetPolygonPolygonGroup(PolygonID);
				RemapGroup.Add(OriginalPolygonGroupID, NewPolygonGroupID);
			}
		}
		//Remap the polygon group with the correct ID
		MeshDescription->RemapPolygonGroups(RemapGroup);
	}
}

FMeshDescriptionArrayAdapter::FRawPoly FMeshDescriptionArrayAdapter::GetRawPoly(const size_t FaceNumber, int32& OutMeshIdx, int32& OutInstanceIdx, int32& OutLocalFaceNumber, const ERawPolyValues RawPolyValues ) const
{
	checkSlow(FaceNumber < PolyCount);

	int32 MeshIdx, InstanceIdx, LocalFaceNumber;

	const FMeshDescriptionAttributesGetter* AttributesGetter = nullptr;
	const FMeshDescription& RawMesh = GetRawMesh(FaceNumber, MeshIdx, InstanceIdx, LocalFaceNumber, &AttributesGetter);
	check(AttributesGetter);
	OutMeshIdx = MeshIdx;
	OutInstanceIdx = InstanceIdx;
	OutLocalFaceNumber = LocalFaceNumber;

	checkSlow(LocalFaceNumber < AttributesGetter->TriangleCount);

	FRawPoly RawPoly;
	RawPoly.MeshIdx = MeshIdx;
	FPolygonID PolygonID(LocalFaceNumber);
	RawPoly.FaceMaterialIndex = RawMesh.GetPolygonPolygonGroup(PolygonID).GetValue();
	RawPoly.FaceSmoothingMask = AttributesGetter->FaceSmoothingMasks[LocalFaceNumber];

	const bool bIsMirrored = InstanceIdx != INDEX_NONE ? InstancesTransformArray[MeshIdx][InstanceIdx].GetDeterminant() < 0.f : false;
	const float MulBy = bIsMirrored ? -1.f : 1.f;

	for (const FTriangleID TriangleID : RawMesh.GetPolygonTriangles(PolygonID))
	{
		TArrayView<const FVertexInstanceID> VertexInstanceIDs = RawMesh.GetTriangleVertexInstances(TriangleID);

		if ((RawPolyValues & ERawPolyValues::VertexPositions) == ERawPolyValues::VertexPositions)
		{
			for (int32 i = 0; i < 3; ++i)
			{
				RawPoly.VertexPositions[i] = AttributesGetter->VertexPositions[RawMesh.GetVertexInstanceVertex(VertexInstanceIDs[i])];
			}

			if (InstanceIdx != INDEX_NONE)
			{
				for (int32 i = 0; i < 3; ++i)
				{
					RawPoly.VertexPositions[i] = (FVector3f)InstancesTransformArray[MeshIdx][InstanceIdx].TransformPosition((FVector)RawPoly.VertexPositions[i]);
				}
			}
		}

		if ((RawPolyValues & ERawPolyValues::WedgeTangents) == ERawPolyValues::WedgeTangents)
		{
			for (int32 i = 0; i < 3; ++i)
			{
				RawPoly.WedgeTangentX[i] = AttributesGetter->VertexInstanceTangents[VertexInstanceIDs[i]];
				RawPoly.WedgeTangentY[i] = FVector3f::CrossProduct(AttributesGetter->VertexInstanceNormals[VertexInstanceIDs[i]], AttributesGetter->VertexInstanceTangents[VertexInstanceIDs[i]]).GetSafeNormal() * AttributesGetter->VertexInstanceBinormalSigns[VertexInstanceIDs[i]];
				RawPoly.WedgeTangentZ[i] = AttributesGetter->VertexInstanceNormals[VertexInstanceIDs[i]];
			}

			if (InstanceIdx != INDEX_NONE)
			{
				for (int32 i = 0; i < 3; ++i)
				{
					RawPoly.WedgeTangentX[i] = FVector4f(InstancesAdjointTArray[MeshIdx][InstanceIdx].TransformVector((FVector)RawPoly.WedgeTangentX[i]) * MulBy);
					RawPoly.WedgeTangentY[i] = FVector4f(InstancesAdjointTArray[MeshIdx][InstanceIdx].TransformVector((FVector)RawPoly.WedgeTangentY[i]) * MulBy);
					RawPoly.WedgeTangentZ[i] = FVector4f(InstancesAdjointTArray[MeshIdx][InstanceIdx].TransformVector((FVector)RawPoly.WedgeTangentZ[i]) * MulBy);
				}
			}
		}

		if ((RawPolyValues & ERawPolyValues::WedgeColors) == ERawPolyValues::WedgeColors)
		{
			for (int32 i = 0; i < 3; ++i)
			{
				RawPoly.WedgeColors[i] = FLinearColor(AttributesGetter->VertexInstanceColors[VertexInstanceIDs[i]]).ToFColor(true);
			}
		}

		if ((RawPolyValues & ERawPolyValues::WedgeTexCoords) == ERawPolyValues::WedgeTexCoords)
		{
			for (int32 i = 0; i < 3; ++i)
			{
				// Copy Texture coords
				for (int Idx = 0; Idx < MAX_MESH_TEXTURE_COORDS_MD; ++Idx)
				{
					if (AttributesGetter->VertexInstanceUVs.GetNumChannels() > Idx)
					{
						RawPoly.WedgeTexCoords[Idx][i] = FVector2D(AttributesGetter->VertexInstanceUVs.Get(VertexInstanceIDs[i], Idx));
					}
					else
					{
						RawPoly.WedgeTexCoords[Idx][i] = FVector2D(0.f, 0.f);
					}
				}
			}
		}
	}

	return RawPoly;
}

FMeshDescriptionArrayAdapter::FRawPoly FMeshDescriptionArrayAdapter::GetRawPoly(const size_t FaceNumber, const ERawPolyValues RawPolyValues) const
{
	int32 IgnoreMeshId, IgnoreInstanceIdx, IgnoreLocalFaceNumber;
	return GetRawPoly(FaceNumber, IgnoreMeshId, IgnoreInstanceIdx, IgnoreLocalFaceNumber, RawPolyValues);
}

// protected functions

const FMeshDescription& FMeshDescriptionArrayAdapter::GetRawMesh(const size_t FaceNumber, int32& MeshIdx, int32& InstanceIdx, int32& LocalFaceNumber, const FMeshDescriptionAttributesGetter** OutAttributesGetter) const
{
	// Binary Search into poly offset will help a lot for big meshes
	auto it = std::upper_bound(PolyOffsetArray.begin(), PolyOffsetArray.end(), FaceNumber);
	MeshIdx = std::distance(PolyOffsetArray.begin(), it) -  1;

	// Offset the face number to get the correct index into this mesh.
	LocalFaceNumber = int32(FaceNumber) - PolyOffsetArray[MeshIdx];

	int32 InstanceCount = !InstancesTransformArray.IsEmpty() ? InstancesTransformArray[MeshIdx].Num() : INDEX_NONE;
	if (InstanceCount > 0)
	{
		int32 NumFacesPerInstance = RawMeshArrayData[MeshIdx].TriangleCount;
		InstanceIdx = LocalFaceNumber / NumFacesPerInstance;
		LocalFaceNumber = LocalFaceNumber % NumFacesPerInstance;
	}
	else
	{
		InstanceIdx = INDEX_NONE;
	}

	const FMeshDescription* MeshDescription = RawMeshArray[MeshIdx];

	*OutAttributesGetter = &RawMeshArrayData[MeshIdx];
	
	return *MeshDescription;
}

void FMeshDescriptionArrayAdapter::ComputeAABB(ProxyLOD::FBBox& InOutBBox)
{
	uint32 NumTris = this->polygonCount();
	InOutBBox = ProxyLOD::Parallel_Reduce(ProxyLOD::FIntRange(0, NumTris), ProxyLOD::FBBox(),
		[this](const ProxyLOD::FIntRange& Range, ProxyLOD::FBBox TargetBBox)->ProxyLOD::FBBox
	{
		// loop over faces
		for (int32 f = Range.begin(), F = Range.end(); f < F; ++f)
		{
			openvdb::Vec3d Pos;
			// loop over verts
			for (int32 v = 0; v < 3; ++v)
			{
				this->GetWorldSpacePoint(f, v, Pos);

				TargetBBox.expand(Pos);
			}

		}

		return TargetBBox;

	}, [](const ProxyLOD::FBBox& BBoxA, const ProxyLOD::FBBox& BBoxB)->ProxyLOD::FBBox
	{
		ProxyLOD::FBBox Result(BBoxA);
		Result.expand(BBoxB);

		return Result;
	}

	);
}

// --- FClosestPolyField ----
FClosestPolyField::FClosestPolyField(const FMeshDescriptionArrayAdapter& MeshArray, const openvdb::Int32Grid::Ptr& SrcPolyIndexGrid) :
	RawMeshArrayAdapter(&MeshArray),
	ClosestPolyGrid(SrcPolyIndexGrid) 
{}

FClosestPolyField::FClosestPolyField(const FClosestPolyField& other) :
	RawMeshArrayAdapter(other.RawMeshArrayAdapter),
	ClosestPolyGrid(other.ClosestPolyGrid) 
{}

FClosestPolyField::FPolyConstAccessor::FPolyConstAccessor(const openvdb::Int32Grid* PolyIndexGrid, const FMeshDescriptionArrayAdapter* MeshArrayAdapter) :
	MeshArray(MeshArrayAdapter),
	CAccessor(PolyIndexGrid->getConstAccessor()),
	XForm(&(PolyIndexGrid->transform()))
{
}

FMeshDescriptionArrayAdapter::FRawPoly FClosestPolyField::FPolyConstAccessor::Get(const openvdb::Vec3d& WorldPos, bool& bSuccess) const
{
	checkSlow(MeshArray != NULL);
	const openvdb::Coord ijk = XForm->worldToIndexCellCentered(WorldPos);
	openvdb::Int32 SrcPolyId;
	bSuccess = CAccessor.probeValue(ijk, SrcPolyId);
	// return the first poly if this failed..
	SrcPolyId = (bSuccess) ? SrcPolyId : 0;
	return MeshArray->GetRawPoly(SrcPolyId);
}


FClosestPolyField::FPolyConstAccessor FClosestPolyField::GetPolyConstAccessor() const
{
	checkSlow(RawMeshArrayAdapter != NULL);
	checkSlow(ClosestPolyGrid != NULL);

	return FPolyConstAccessor(ClosestPolyGrid.get(), RawMeshArrayAdapter);
}


