// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER

// Some of the libraries we depend on will re-define the TEXT macro.

#pragma warning(disable : 4005)

#endif

#include "MeshMergeData.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

#include "ProxyLODThreadedWrappers.h"
#include "ProxyLODVertexTypes.h"
#include "ProxyLODOpenVDB.h"

THIRD_PARTY_INCLUDES_START
#include <vector>
THIRD_PARTY_INCLUDES_END

/**
* Various stages in the ProxyLOD pipeline require different internal mesh formats, but
* both input and output data for the ProxyLOD process use the native UE format, FMeshDescription.
*
*  
*
* FMeshDescriptionAdapter:
*	An adapter class to allow a FMeshDescription to be voxelized without actually converting to a new mesh format.
*
* FMeshDescriptionArrayAdapter:
*	An adapter class to allow an array of FMeshDescription items to be viewed as a single mesh.  Primarly to allow
*   multiple items of geometry to be voxelized into a single grid.
* 
* FMixedPolyMesh:
*	Iso-surface extraction from voxelization requires a simple Struct of Arrays that supports
*   only point locations and connectivity using both triangles and quads.
*
* FAOSMesh:
*	The simplifier uses an Array Of Structs format with normal and position stored together in a Vertex Struct.
* 
* FVertexDataMesh:
*	Generation of UVs and tangent space uses a Struct Of Arrays that supports only triangles, but also includes
*	per-vertex tangent space and UVs channels. 
*
*
*/

/**
* Utility to resize an array with uninitialized data.
* NB: this destroys any content.
*/
template <typename ValueType>
void ResizeArray(TArray<ValueType>& Array, int32 Size)
{
	Array.Empty(Size);
	Array.AddUninitialized(Size);
}

/**
* Utility to resize an array with constructor initialized data.
* NB: this destroys any content.
*/
template <typename ValueType>
void ResizeInializedArray(TArray<ValueType>& Array, int32 Size)
{
	if (Size == 0)
	{
		TArray<ValueType> EmptyVector;
		Swap(EmptyVector, Array);
	}
	else
	{

		TArray<ValueType> Tmp;
		Tmp.InsertDefaulted(0, Size);

		Swap(Array, Tmp);
	}
}
/**
* Mesh type that holds minimal required data for openvdb iso-surface extraction 
* interface. Implements the openvdb MeshDataAdapter interface.
* 
* NB: std::vector required by the openvdb interface.
*/
struct FMixedPolyMesh
{
	std::vector<openvdb::Vec3s> Points;
	std::vector<openvdb::Vec4I> Quads;
	std::vector<openvdb::Vec3I> Triangles;
	openvdb::math::Transform	Transform;

	// ~MeshDataAdapter Interface Begin
	// https://www.openvdb.org/documentation/doxygen/interfaceMeshDataAdapter.html
	
	// Total number of polygons
	size_t polygonCount() const
	{
		return Quads.size() + Triangles.size();
	}

	// Total number of points (vertex locations)
	size_t pointCount() const
	{
		return Points.size();
	}

	// Vertex count for polygon n
	size_t vertexCount(size_t n) const
	{
		return PolygonIsQuad(n) ? 4 : 3;
	}

	// Return position pos in local grid index space for polygon n and vertex v
	void getIndexSpacePoint(size_t n, size_t v, openvdb::Vec3d& pos) const
	{
		if (PolygonIsQuad(n))
		{
			pos = Points[Quads[GetQuadIndex(n)][v]];
		}
		else
		{
			pos = Points[Triangles[GetTriangleIndex(n)][v]];
		}

		pos = Transform.worldToIndex(pos);
	}
	// ~MeshDataAdapter Interface End

private:
	bool PolygonIsQuad(size_t n) const
	{
		return n < Quads.size();
	}

	bool PolygonIsTriangle(size_t n) const
	{
		return n >= Triangles.size();
	}

	size_t GetQuadIndex(size_t n) const
	{
		return n;
	}

	size_t GetTriangleIndex(size_t n) const
	{
		return n - Quads.size();
	}
};

/**
* Vertex-based mesh type that works well with the DirectX  mesh tools. 
* For this use case there are no split normals, 
* i.e. Normal.Num() == Points.Num();
*      UVs.Num() == Points.Num();
*/
struct FVertexDataMesh
{
	TArray<uint32>    Indices;
	TArray<FVector3f>   Points;

	TArray<FVector3f>   Normal;
	TArray<FVector3f>   Tangent;
	TArray<FVector3f>   BiTangent;

	// Optional. Used for ray shooting when transfering materials.
	TArray<FVector3f>   TransferNormal;

	// stores information about the tangentspace
	// 1 = right handed.  -1 left handed
	TArray<int32>     TangentHanded;

	TArray<FVector2f> UVs;

	// Per-face frequency
	TArray<FColor>    FaceColors;
	TArray<uint32>    FacePartition;
};


/**
* A Triangle Mesh Array Of Structs (AOS) for the vertex data.  This should work with the simplification code.
*/
template <typename SimplifierVertexType>
class TAOSMesh
{
public:

	typedef SimplifierVertexType  VertexType;

	TAOSMesh();
	TAOSMesh(int32 VertCount, int32 FaceCount); 

	~TAOSMesh();
	
	// reset the mesh to size zero, deleting content.
	void Empty();
	
	// resize the mesh, deleting the current content.
	void Resize(int32 VertCount, int32 FaceCount);

	// set the real vertex & index count after duplicate removal
	void SetVertexAndIndexCount(uint32 VertCount, uint32 IndexCount);

	// Swap content with an existing mesh of the same type.
	void Swap(TAOSMesh& other);
	

	// Get the indices that correspond to this face.
	openvdb::Vec3I GetFace(int32 FaceNumber) const;
	
	// Get an array of positions only.
	void GetPosArray(std::vector<FVector3f>& PosArray) const;
	

	// This method is for testing only.  Not designed for perf.
	int32 ComputeDegenerateTriCount() const;
	

	bool IsEmpty() const { return (NumVertexes == 0 || NumIndexes == 0); }
	uint32 GetNumVertexes() const { return NumVertexes; }
	uint32 GetNumIndexes()  const { return NumIndexes; }

	// unfettered public access.
	VertexType* Vertexes;
	uint32*     Indexes;

private:

	uint32 NumVertexes;
	uint32 NumIndexes;

	// No!

	TAOSMesh(const TAOSMesh& other);
};



/**
* Array of structs mesh composed of vertices that are compatible with the simplifier.
*/
typedef TAOSMesh<FPositionNormalVertex>  FAOSMesh;
typedef TAOSMesh<FPositionOnlyVertex>    FPosSimplifierMesh;


/**
*  Mesh adapter class that implements the needed API for the methods in openvdb that convert
*  polygonal mesh to a signed distance field.
*/
class FMeshDescriptionAdapter
{
public:
	/**
	* Constructor 
	*/
	FMeshDescriptionAdapter(const FMeshDescription& InRawMesh, const openvdb::math::Transform& InTransform);
	FMeshDescriptionAdapter(const FMeshDescriptionAdapter& other);

	// Total number of polygons
	size_t polygonCount() const;

	// Total number of points (vertex locations)
	size_t pointCount() const;
	
	// Vertex count for polygon n: currently FMeshDescription is just triangles.
	size_t vertexCount(size_t n) const { return 3; }

	// Return position pos in local grid index space for polygon n and vertex v
	void getIndexSpacePoint(size_t FaceNumber, size_t CornerNumber, openvdb::Vec3d& pos) const;
	
	/**
	* The transform used to map between the physical space of the mesh and the voxel space.
	*/
	const openvdb::math::Transform& GetTransform() const
	{
		return Transform;
	}

private:

	// Pointer to the raw mesh we are wrapping
	const FMeshDescription* RawMesh;

	

	//////////////////////////////////////////////////////////////////////////
	//Cache data
	void InitializeCacheData();
	uint32 TriangleCount;
	TVertexAttributesConstRef<FVector3f> VertexPositions;
	// Local version of the index array.  The FMeshDescription doesn't really have one.
	TArray<FVertexInstanceID> IndexBuffer;



	// Transform used to convert the mesh space into the index space used by voxelization
	const openvdb::math::Transform Transform;
};

namespace ProxyLOD
{
	typedef openvdb::math::BBox<openvdb::Vec3s>   FBBox;
}

// Used to filter initialization of required values only
enum class ERawPolyValues : uint8
{
	None            = 0,
	WedgeColors     = (1 << 0),
	WedgeTexCoords  = (1 << 1),
	WedgeTangents   = (1 << 2),
	VertexPositions = (1 << 3),
	All             = 0xff
};

ENUM_CLASS_FLAGS(ERawPolyValues);

/**
* Adapter class that make an array of raw meshes appear to be a single mesh 
* with implementation of the methods to interact with the templated openvdb mesh to volume code. 
* 
* NB: This is only an adapter class that holds pointers to external objects.  It does not manage the lifetimes of those objects,
*
*/
class FMeshDescriptionArrayAdapter
{
public:
	// OpenVDB MeshDataAdapter Interface
	
	// Total number of polygons managed by this class.
	size_t polygonCount() const
	{
		return PolyCount;
	}

	// Total number of points (positions) managed by this class
	size_t pointCount() const
	{
		return PointCount;
	}

	// Vertex count for polygon n: currently FMeshDescription is just triangles.
	size_t vertexCount(size_t n) const
	{
		return 3;
	}

	// Return position pos in local grid index space for polygon (face number) n and vertex (conrner number) v
	void getIndexSpacePoint(size_t FaceNumber, size_t CornerNumber, openvdb::Vec3d& pos) const;

public:
	FMeshDescriptionArrayAdapter(const TArray<FMeshMergeData>& InMergeDataArray, const openvdb::math::Transform::Ptr InTransform);
	FMeshDescriptionArrayAdapter(const TArray<const FMeshMergeData*>& InMergeDataPtrArray);
	FMeshDescriptionArrayAdapter(const TArray<const FInstancedMeshMergeData*>& InMergeDataPtrArray);
	FMeshDescriptionArrayAdapter(const TArray<FMeshMergeData>& InMergeDataArray);
	FMeshDescriptionArrayAdapter(const TArray<FInstancedMeshMergeData>& InMergeDataArray);
	
	// copy constructor 
	FMeshDescriptionArrayAdapter(const FMeshDescriptionArrayAdapter& other);

	//Destructor
	~FMeshDescriptionArrayAdapter();
	
	// Return position for polygon (face number) n and vertex (corner number) v
	void GetWorldSpacePoint(size_t FaceNumber, size_t CornerNumber, openvdb::Vec3d& pos) const;

	// Access to the FMeshMergeData data elements in the array.
	const FMeshMergeData& GetMeshMergeData(uint32 Idx) const;

	//Set the FMeshMergeData material into the FMeshDescription
	void UpdateMaterialsID();
	
	/**
	* Triangle class that contains the information associated with a single triangle
	* in an FMeshDescription.
	*  
	*  NB: The FMeshDescription is a struct of arrays, this basically converts the data for a single 
	*      poly into a struct.
	*/
	class FRawPoly
	{
	public:
		int32 MeshIdx;

		int32   FaceMaterialIndex;

		uint32  FaceSmoothingMask;
		FVector3f VertexPositions[3];

		FVector3f   WedgeTangentX[3];
		FVector3f   WedgeTangentY[3];
		FVector3f   WedgeTangentZ[3];

		FVector2D WedgeTexCoords[MAX_MESH_TEXTURE_COORDS_MD][3];

		FColor WedgeColors[3];
	};

	class FMeshDescriptionAttributesGetter
	{
	public:
		FMeshDescriptionAttributesGetter(const FMeshDescription* MeshDescription)
		{
			FStaticMeshConstAttributes Attributes(*MeshDescription);

			VertexPositions = Attributes.GetVertexPositions();
			VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
			VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
			VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
			VertexInstanceColors = Attributes.GetVertexInstanceColors();
			VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

			TriangleCount = MeshDescription->Triangles().Num();
			FaceSmoothingMasks.AddZeroed(TriangleCount);
			FStaticMeshOperations::ConvertHardEdgesToSmoothGroup(*MeshDescription, FaceSmoothingMasks);
		}
		TVertexAttributesConstRef<FVector3f> VertexPositions;
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals;
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents;
		TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns;
		TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors;
		TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs;
		int32 TriangleCount;
		TArray<uint32> FaceSmoothingMasks;
	};

	/**
	* Returns a copy of the data associated with this poly in the form of a struct.
	* 
	* @param FaceNumber          The triangle Id when treating all the meshes as a single mesh
	* @param OutMeshIdx          The Id of the actual mesh that owns this poly
	* @param OutInstanceIdx      The instance index, if any, or INDEX_NONE
	* @param OutLocalFaceNumber  The Id within that mesh of this poly
	* @param RawPolyValues       Reduce computations by specifying which values will be used from FRawPoly.
	*
	* @return  A copy of the raw mesh data associated with this poly.
	*/
	FMeshDescriptionArrayAdapter::FRawPoly GetRawPoly(const size_t FaceNumber, int32& OutMeshIdx, int32& OutInstanceIdx, int32& OutLocalFaceNumber, const ERawPolyValues RawPolyValues = ERawPolyValues::All ) const;
	
	/**
	* Returns a copy of the data associated with this poly in the form of a struct.
	*
	* @param FaceNumber          The triangle Id when treating all the meshes as a single mesh
	* @param RawPolyValues       Reduce computations by specifying which values will be used from FRawPoly.
	*
	* @return  A copy of the raw mesh data associated with this poly.
	*/
	FMeshDescriptionArrayAdapter::FRawPoly GetRawPoly(const size_t FaceNumber, const ERawPolyValues RawPolyValues = ERawPolyValues::All ) const;

	/**
	* The transform used to map between the physical space of the mesh and the voxel space.
	*/
	const openvdb::math::Transform& GetTransform() const 
	{ 
		return *Transform; 
	}

	void SetTransform(const openvdb::math::Transform::Ptr Xform)
	{
		Transform = Xform;
	}
	/**
	* Axis aligned bounding box that holds all the polys mananged by this class.
	*/
	const ProxyLOD::FBBox& GetBBox() const 
	{ 
		return BBox; 
	}

protected:
	void SetupInstances(int32 MeshCount, TFunctionRef<const FInstancedMeshMergeData* (uint32 Index)> GetMeshFunction);

	void Construct(int32 MeshCount, TFunctionRef<const FMeshMergeData* (uint32 Index)> GetMeshFunction);
	void Construct(int32 MeshCount, TFunctionRef<const FMeshMergeData* (uint32 Index)> GetMeshFunction, TFunctionRef<int32(uint32 Index)> GetInstanceCountFunction);

	const FMeshDescription& GetRawMesh(const size_t FaceNumber, int32& MeshIdx, int32& InstanceIdx, int32& LocalFaceNumber, const FMeshDescriptionAttributesGetter** OutAttributesGetter) const;

	void ComputeAABB(ProxyLOD::FBBox& InOutBBox);
protected:

	openvdb::math::Transform::Ptr Transform;

	size_t PointCount;
	size_t PolyCount;
	
	ProxyLOD::FBBox   BBox;

	std::vector<size_t>                      PolyOffsetArray;
	std::vector<FMeshDescription*>           RawMeshArray;
	TArray<TArray<FTransform>>               InstancesTransformArray;
	TArray<TArray<FMatrix>>                  InstancesAdjointTArray;

	// Use TArray because we need SetNumUninitialized
	TArray<FMeshDescriptionAttributesGetter> RawMeshArrayData;

	std::vector<const FMeshMergeData*>       MergeDataArray;

	// Need to build local index buffers for each mesh because the FMeshDescription doesn't natively have the construct.
	std::vector<std::vector<FVertexInstanceID>> IndexBufferArray;

};

/**
* Class that associates reference geometry (in the form of a FMeshDescriptionArrayAdapter) and a sparse index grid.
*
* In standard use, the sparse index grid will hold the Id of the closest poly to the center of each voxel
* in the voxel.
*
* Example use:
*
*  openvdb::FloatGrid::Ptr SDFVolume;
*  openvdb::Int32Grid::Ptr SrcPolyIndexGrid = openvdb::Int32Grid::create();
*  ProxyLOD::MeshArrayToSDFVolume(MeshAdapter, SDFVolume, SrcPolyIndexGrid.get());

*  // Create an object that allows for closest poly queries against the source mesh.
*  FClosestPolyField::Ptr SrcReference = FClosestPolyField::create(MeshAdapter, SrcPolyIndexGrid);
*
* NB: this does not manage the life time of the underlying data.
*
*/
class FClosestPolyField
{
public:

	typedef std::shared_ptr<FClosestPolyField>        Ptr;
	typedef std::shared_ptr<const FClosestPolyField>  ConstPtr;

	static ConstPtr create(const FMeshDescriptionArrayAdapter& MeshArray, const openvdb::Int32Grid::Ptr SrcPolyIndexGrid)
	{
		return ConstPtr(new FClosestPolyField(MeshArray, SrcPolyIndexGrid));
	}

	FClosestPolyField(const FMeshDescriptionArrayAdapter& MeshArray, const openvdb::Int32Grid::Ptr& SrcPolyIndexGrid);

	FClosestPolyField(const FClosestPolyField& other);
		

	/**
	* Accessor into the Closest Poly Field, thread safe - but does not manage the lifetime 
	* of any of the underlying objects (Meshes or index grid)
	*/
	class FPolyConstAccessor
	{
	public:
		FPolyConstAccessor(const openvdb::Int32Grid* PolyIndexGrid, const FMeshDescriptionArrayAdapter* MeshArrayAdapter);

		/** 
		* Return the closest poly, also let us know if the query failed.
		*
		* @param WorldPos    Query location in physical space - will be snapped to a voxel center internally. 
		* @param bSuccess    On return will be 'true' if the the voxel contained valid data, 'false' otherwise.
		*
		* @return FRawyPoly  Stuct holding all the data associated with the closest poly in the reference geometry.
		*/
		FMeshDescriptionArrayAdapter::FRawPoly  Get(const openvdb::Vec3d& WorldPos, bool& bSuccess) const;
		FMeshDescriptionArrayAdapter::FRawPoly  Get(const FVector3f& WorldPos, bool& bSuccess) const
		{
			return this->Get(openvdb::Vec3d(WorldPos.X, WorldPos.Y, WorldPos.Z), bSuccess);
		}

	private:
		const FMeshDescriptionArrayAdapter*       MeshArray;
		openvdb::Int32Grid::ConstAccessor CAccessor;
		const openvdb::math::Transform*   XForm;

	};

	/**
	* For threaded work, each thread should have its own PolyConstAccessor
	*/
	FClosestPolyField::FPolyConstAccessor GetPolyConstAccessor() const;
	

	/**
	* The voxel size used to represent the closest poly grid.
	* 
	* @return double precision size of voxel.
	*/ 
	double GetVoxelSize() const { return ClosestPolyGrid->voxelSize()[0]; }

	/**
	* Access to the underlying reference geometry.
	*/
	const FMeshDescriptionArrayAdapter& MeshAdapter() const { return *RawMeshArrayAdapter; }

private:

	const FMeshDescriptionArrayAdapter* RawMeshArrayAdapter;
	openvdb::Int32Grid::ConstPtr   ClosestPolyGrid;
};


 // --- Implementation of templated code. ---


template <typename SimplifierVertexType>
TAOSMesh<SimplifierVertexType>::TAOSMesh() :
	Vertexes(NULL),
	Indexes(NULL),
	NumVertexes(0),
	NumIndexes(0)
{}

template <typename SimplifierVertexType>
TAOSMesh<SimplifierVertexType>::TAOSMesh(int32 VertCount, int32 FaceCount) :
	Vertexes(NULL),
	Indexes(NULL),
	NumVertexes(0),
	NumIndexes(0)
{
	this->Resize(VertCount, FaceCount);
}

template <typename SimplifierVertexType>
TAOSMesh<SimplifierVertexType>::~TAOSMesh()
{
	this->Empty();
}

template <typename SimplifierVertexType>
void TAOSMesh<SimplifierVertexType>::Empty()
{
	if (Vertexes) delete[] Vertexes;
	if (Indexes)  delete[] Indexes;

	Vertexes = NULL;
	Indexes = NULL;
	NumVertexes = 0;
	NumIndexes = 0;
}

template <typename SimplifierVertexType>
void TAOSMesh<SimplifierVertexType>::Resize(int32 VertCount, int32 FaceCount)
{
	check(VertCount > -1); check(FaceCount > -1);

	this->Empty();

	if (FaceCount > 0 && VertCount > 0)
	{
		NumVertexes = VertCount;
		NumIndexes = FaceCount * 3;

		Vertexes = new VertexType[VertCount];
		Indexes = new uint32[NumIndexes];
	}

}

template <typename SimplifierVertexType>
void TAOSMesh<SimplifierVertexType>::SetVertexAndIndexCount(uint32 VertCount, uint32 IndexCount)
{
	check(VertCount <= NumVertexes);
	check(IndexCount <= NumIndexes);

	NumVertexes = VertCount;
	NumIndexes = IndexCount;
}

template <typename SimplifierVertexType>
void TAOSMesh<SimplifierVertexType>::Swap(TAOSMesh& other)
{
	// Swap sizes
	{
		uint32 Tmp = other.NumVertexes;
		other.NumVertexes = NumVertexes;
		NumVertexes = Tmp;

		Tmp = other.NumIndexes;
		other.NumIndexes = NumIndexes;
		NumIndexes = Tmp;
	}

	// Swap Vertex Pointer
	{
		VertexType* TmpVp = other.Vertexes;
		other.Vertexes = Vertexes;
		Vertexes = TmpVp;
	}

	// Sway index pointer
	{
		uint32* TmpIdp = other.Indexes;
		other.Indexes = Indexes;
		Indexes = TmpIdp;
	}

}

template <typename SimplifierVertexType>
openvdb::Vec3I TAOSMesh<SimplifierVertexType>::GetFace(int32 FaceNumber) const
{
	check(FaceNumber > -1); check(uint32(FaceNumber * 3) < NumIndexes);

	uint32 offset = FaceNumber * 3;
	return openvdb::Vec3I(Indexes[offset], Indexes[offset + 1], Indexes[offset + 2]);
}

template <typename SimplifierVertexType>
void TAOSMesh<SimplifierVertexType>::GetPosArray(std::vector<FVector3f>& PosArray) const
{
	// resize the target array
	PosArray.resize(NumVertexes);

	// copy the data over.
	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumVertexes), [this, &PosArray](const ProxyLOD::FUIntRange& Range)
	{
		FVector3f* Pos = PosArray.data();
		FPositionNormalVertex* VertexArray = this->Vertexes;


		for (uint32 r = Range.begin(), R = Range.end(); r < R; ++r)
		{
			Pos[r] = VertexArray[r].GetPos();
		}
	});
}

template <typename SimplifierVertexType>
int32 TAOSMesh<SimplifierVertexType>::ComputeDegenerateTriCount() const
{
	int32 DegenerateTriCount = 0;
	uint32 NumTris = GetNumIndexes() / 3;
	for (uint32 i = 0; i < NumTris; ++i)
	{
		uint32 Offset = i * 3;
		uint32 Idx[3] = { Indexes[Offset], Indexes[Offset + 1], Indexes[Offset + 2] };
		FVector Tri[3] = { Vertexes[Idx[0]].GetPos(), Vertexes[Idx[1]].GetPos(), Vertexes[Idx[2]].GetPos() };

		const FVector NormalDir = (Tri[2] - Tri[0]) ^ (Tri[1] - Tri[0]);

		if (NormalDir.SizeSquared() == 0.0f)
		{
			DegenerateTriCount++;
		}
	}

	return DegenerateTriCount;
}
