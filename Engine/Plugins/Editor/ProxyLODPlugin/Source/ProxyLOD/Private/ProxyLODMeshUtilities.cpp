// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODMeshUtilities.h"

#include "CoreMinimal.h"

#include "ProxyLODkDOPInterface.h"
#include "ProxyLODMeshConvertUtils.h" // ComputeNormal

#include "Modules/ModuleManager.h"
#include "MeshUtilities.h" // IMeshUtilities

THIRD_PARTY_INCLUDES_START
#include <DirectXMesh/DirectXMesh.h>
THIRD_PARTY_INCLUDES_END

#include "StaticMeshAttributes.h"
#include "TriangleTypes.h"

#include <vector>
#include <map>
#include <unordered_map>

#define LOCTEXT_NAMESPACE "ProxyLODMeshUtilities"

#ifndef  PROXYLOD_CLOCKWISE_TRIANGLES
	#define PROXYLOD_CLOCKWISE_TRIANGLES  1
#endif
// Compute a tangent space for a FMeshDescription 

void ProxyLOD::ComputeTangentSpace(FMeshDescription& RawMesh, const bool bRecomputeNormals)
{
	// Static meshes always blend normals of overlapping corners.
	EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals | EComputeNTBsFlags::IgnoreDegenerateTriangles | EComputeNTBsFlags::Tangents | EComputeNTBsFlags::UseMikkTSpace;
	FStaticMeshOperations::ComputeTangentsAndNormals(RawMesh, ComputeNTBsOptions);
}

// Calls into the direxXMesh library to compute the per-vertex normal, by default this will weight by area.
// Note this is different than computing on the raw mesh, which can result in per-index tangent space.

void ProxyLOD::ComputeVertexNormals(FVertexDataMesh& InOutMesh, const ENormalComputationMethod Method)
{
	// Note:
	// This code relies on the fact that a FVector3f can be cast as a XMFLOAT3, and a FVector2D can be cast as a XMFLOAT2

	// Data from the existing mesh

	const DirectX::XMFLOAT3* Pos = (DirectX::XMFLOAT3*) (InOutMesh.Points.GetData());
	const size_t NumVerts = InOutMesh.Points.Num();
	const size_t NumFaces = InOutMesh.Indices.Num() / 3;
	uint32* indices = InOutMesh.Indices.GetData();

	auto& NormalArray = InOutMesh.Normal;
	ResizeArray(NormalArray, NumVerts);

	DirectX::XMFLOAT3* Normals = (DirectX::XMFLOAT3*)NormalArray.GetData();

	// Default is weight by angle.
	DWORD NormalFlags = 0;
	switch (Method)
	{
		case ENormalComputationMethod::AngleWeighted:
			NormalFlags |= DirectX::CNORM_FLAGS::CNORM_DEFAULT;
			break;
		case ENormalComputationMethod::AreaWeighted:
			NormalFlags |= DirectX::CNORM_FLAGS::CNORM_WEIGHT_BY_AREA;
			break;
	
		case ENormalComputationMethod::EqualWeighted:
			NormalFlags |= DirectX::CNORM_FLAGS::CNORM_WEIGHT_EQUAL;
			break;
		

		default:
			NormalFlags |= DirectX::CNORM_FLAGS::CNORM_DEFAULT;
			break;
	}


#if (PROXYLOD_CLOCKWISE_TRIANGLES == 1)
	NormalFlags |= DirectX::CNORM_FLAGS::CNORM_WIND_CW;
#endif
	DirectX::ComputeNormals(indices, NumFaces, Pos, NumVerts, NormalFlags, Normals);

}


// Calls into the direxXMesh library to compute the per-vertex tangent and bitangent, optionally recomputes the normal.
// Note this is different than computing on the raw mesh, which can result in per-index tangent space.

void ProxyLOD::ComputeTangentSpace( FVertexDataMesh& InOutMesh, const bool bRecomputeNormals)
{

	// Note:
	// This code relies on the fact that a FVector3f can be cast as a XMFLOAT3, and a FVector2D can be cast as a XMFLOAT2

	// Data from the existing mesh

	const DirectX::XMFLOAT3* Pos = (DirectX::XMFLOAT3*) (InOutMesh.Points.GetData());
	const size_t NumVerts = InOutMesh.Points.Num();
	const size_t NumFaces = InOutMesh.Indices.Num() / 3;
	uint32* indices = InOutMesh.Indices.GetData();

	// Optional computation of the normal
	if (bRecomputeNormals)
	{
		auto& NormalArray = InOutMesh.Normal;
		ResizeArray(NormalArray, NumVerts);

		DirectX::XMFLOAT3* Normals = (DirectX::XMFLOAT3*)NormalArray.GetData();

		// Default is weight by angle.
		DWORD NormalFlags = 0;
		NormalFlags |= DirectX::CNORM_FLAGS::CNORM_DEFAULT;
#if (PROXYLOD_CLOCKWISE_TRIANGLES == 1)
		NormalFlags |= DirectX::CNORM_FLAGS::CNORM_WIND_CW;
#endif
		DirectX::ComputeNormals(indices, NumFaces, Pos, NumVerts, NormalFlags, Normals);
	}

	// Compute the tangent and bitangent

	const auto& NormalArray = InOutMesh.Normal;
	const DirectX::XMFLOAT3* Normals = (const DirectX::XMFLOAT3*)NormalArray.GetData();

	const auto& UVArray = InOutMesh.UVs;
	const DirectX::XMFLOAT2* TexCoords = (const DirectX::XMFLOAT2*)UVArray.GetData();


	auto& TangentArray = InOutMesh.Tangent;
	ResizeArray(TangentArray, NumVerts);

	auto& BiTangentArray = InOutMesh.BiTangent;
	ResizeArray(BiTangentArray, NumVerts);

	//DirectX::ComputeTangentFrame(indices, NumFaces, Pos, Normals, TexCoords, NumVerts, (DirectX::XMFLOAT3*)TangentArray.GetData(), (DirectX::XMFLOAT3*)BiTangentArray.GetData());

	// Compute the tangent and bitangent frame and record the handedness.
	DirectX::XMFLOAT4* TangentX = new DirectX::XMFLOAT4[NumVerts];
	DirectX::ComputeTangentFrame(indices, NumFaces, Pos, Normals, TexCoords, NumVerts, TangentX, (DirectX::XMFLOAT3*)BiTangentArray.GetData());

	auto& TangentHanded = InOutMesh.TangentHanded;
	ResizeArray(TangentHanded, NumVerts);

	for (int32 v = 0; v < NumVerts; ++v)
	{
		// the handedness result was stored in TangentX.w by ::ComputeTangentFrame

		TangentHanded[v] = (TangentX[v].w > 0) ? 1 : -1;

		TangentArray[v] = FVector3f(TangentX[v].x, TangentX[v].y, TangentX[v].z);
	}

	if (TangentX) delete[] TangentX;
}

class FVertexIdToFaceIdAdjacency
{
public:
	typedef TArray<int32, TInlineAllocator<16>> FaceList; 

	FVertexIdToFaceIdAdjacency(const uint32* Indices, const int32 NumIndices, const int32 InNumVerts)
	{
		typedef uint32 VertIdxType;
		check(NumIndices % 3 == 0);

		checkSlow(InNumVerts > -1);
		checkSlow(NumIndices > -1);

		if (NumIndices == 0 && InNumVerts == 0) return;

		// The number of triangles

		const int32 NumTris = NumIndices / 3;

		// Allocate the result array.  Note the default constructor has to be called on each ellement

		{
			ResizeInializedArray(VertexToFaces, InNumVerts);
		}

		// Construct a list of faces that are adjacent to each vertex


		for (int32 f = 0; f < NumTris; ++f)
		{
			// Register this face with the correct verts

			const int32 TriOffset = 3 * f;  
			checkSlow(TriOffset + 2 < NumIndices); 
			const VertIdxType VertId[3] = { Indices[TriOffset + 0], Indices[TriOffset + 1], Indices[TriOffset + 2] };

			for (int v = 0; v < 3; ++v) {
				checkSlow(VertId[v] < (uint32)InNumVerts);
				// NB: For debug, make this AddUnique
				VertexToFaces[VertId[v]].Add(f);
			}
		}
	}

	/**
	* Find the faces that are adjacent to the edge Vert0-Vert1.
	*
	* @param Vert0 - one vertex of the edge
	* @param Vert1 - other vertex of the edge
	*
	* @param AdjFaces - The Ids of faces that are adjacent to this edge.

	* @return - Returns true if the edge is locally manifold (no more than 2 adj faces).  Otherwise false.
	*/
	bool FindAdjacentFaces(const uint32 Vert0, const uint32 Vert1, FaceList& AdjFaces) const 
	{

		// initialize 


		// lists of faces for each vert

		const FaceList& FacesAdjacentToV0 = this->VertexToFaces[Vert0];
		const FaceList& FacesAdjacentToV1 = this->VertexToFaces[Vert1];


		bool bManifold = true;

		for (auto V0FaceId : FacesAdjacentToV0)
		{
			{
				int32 Result = FacesAdjacentToV1.Find(V0FaceId);
				if (Result != INDEX_NONE)
				{
					AdjFaces.Add(V0FaceId);
				}
			}
		}

		if (AdjFaces.Num() == 0 || AdjFaces.Num() > 2)
		{
			bManifold = false;
		}

		return bManifold;
	};

	// Identify the adjacent triangles to a given vertex.
	// List of faces adjacent to each vertex
	// ConcurrentFaceList& = VertexToFaces[VertexId] 

	TArray<FaceList>  VertexToFaces;

private:
	FVertexIdToFaceIdAdjacency();
};

class FAdjacencyData : public FVertexIdToFaceIdAdjacency
{

public:
	typedef TArray<int32, TInlineAllocator<16>> FaceList;
	typedef TArray<int32> EdgeList;

	class SimpleEdge
	{
	public:
		SimpleEdge() { Verts[0] = 0; Verts[1] = 0; };


		SimpleEdge(const uint32 VertA, const uint32 VertB)
		{
			if (VertA > VertB)
			{
				Verts[0] = VertB;
				Verts[1] = VertA;
			}
			else
			{
				Verts[0] = VertA;
				Verts[1] = VertB;
			}
		}

		SimpleEdge(const SimpleEdge& other)
		{
			Verts[0] = other.Verts[0];
			Verts[1] = other.Verts[1];
		}
		SimpleEdge& operator=(const SimpleEdge& other)
		{
			Verts[0] = other.Verts[0];
			Verts[1] = other.Verts[1];
			return *this;
		}

		bool operator==(const SimpleEdge& other) const
		{
			return (Verts[0] == other.Verts[0] && Verts[1] == other.Verts[1]);
		}

		bool operator<(const SimpleEdge& other) const
		{
			return (other.Verts[0] > Verts[0] || (other.Verts[0] == Verts[0] && other.Verts[1] > Verts[1]));
		}

		uint32 Verts[2];
	};

	struct SimpleEdgeComparator
	{
		bool operator()(const SimpleEdge& lhs, const SimpleEdge& rhs) const
		{
			return lhs.operator<(rhs);
		}
	};

	class FaceAssociation
	{
	public:
		FaceAssociation() :
			FaceId(-1),
			NextId(-1),
			LastId(-1)
		{}

		FaceAssociation(const int32 Id) :
			FaceId(Id),
			NextId(Id),
			LastId(Id)
		{}

		FaceAssociation(const FaceAssociation& other) :
			FaceId(other.FaceId),
			NextId(other.NextId),
			LastId(other.LastId)
		{}

		// For a correctly links group of faces  LastId <= FaceId <=NextId; 
		int32 FaceId;
		int32 NextId;
		int32 LastId;
	};

	FAdjacencyData(const uint32* Indices, const int32 NumIndices, const int32 InNumVerts) :
		FVertexIdToFaceIdAdjacency(Indices, NumIndices, InNumVerts)
	{


		int32 NumVerts = InNumVerts;

		// The number of triangles

		const int32 NumTris = NumIndices / 3;
		check(NumIndices % 3 == 0);


		std::map< SimpleEdge, FaceList, SimpleEdgeComparator >  EdgeToFaceMap;


		// Make a map of edges to faces.

		for (int32 faceIdx = 0; faceIdx < NumTris; ++faceIdx)
		{
			int32 offset = 3 * faceIdx;

			// Add this face to the 3 edges
			for (int32 v = 0; v < 3; ++v)
			{
				int32 nv = (v + 1) % 3; // next vertex
				checkSlow(nv < NumVerts);  

				SimpleEdge  Edge(Indices[offset + v], Indices[offset + nv]);
				auto Search = EdgeToFaceMap.find(Edge);
				if (Search != EdgeToFaceMap.end())
				{
					auto& Faces = Search->second;
					Faces.Add(faceIdx);
				}
				else
				{
					EdgeToFaceMap[Edge].Add(faceIdx);
				}
			}
		}

		// Make an array of edges and a corresponding array of faces.
		const int32 NumEdges = (int32)(EdgeToFaceMap.size());
		{
			ResizeArray(EdgeArray, NumEdges);
		}
		{
			ResizeInializedArray(EdgeToFaces, NumEdges);
		}

		{
			int32 offset = 0;
			for (auto iter = EdgeToFaceMap.begin(); iter != EdgeToFaceMap.end(); ++iter)
			{
				EdgeArray[offset] = iter->first;

				Swap(EdgeToFaces[offset], iter->second);

				offset++;

			}
		}

		// Allocate an array: Index by VertexId, Holds Adj Edges
		{
			ResizeInializedArray(VertexToEdges, NumVerts);
		}

		// make map of vertex to edge
		for (int32 edgeIdx = 0; edgeIdx < NumEdges; ++edgeIdx)
		{
			const auto& Edge = EdgeArray[edgeIdx];
			checkSlow(Edge.Verts[0] < Edge.Verts[1]); 
			for (int32 v = 0; v < 2; ++v)
			{
				uint32 VertIdx = Edge.Verts[v];
				VertexToEdges[VertIdx].Add(edgeIdx);
			}
		}

	}

	/**
	* Find the faces that are adjacent to the edge Vert0-Vert1.
	*
	* @param Edge - The edge in question
	*
	* @param AdjFaces - The Ids of faces that are adjacent to this edge.
	*                   If only one face is adjacent (e.g. a boundary) then -1 will
	*                   be returned for the other.  If no faces are found then -1, will be returned for each
	*
	* @return - Returns true if the edge is locally manifold (no more than 2 adj faces).  Otherwise false.
	*/
	bool FindAdjacentFaces(const SimpleEdge& Edge, FaceList& AdjFaces) const
	{
		return FVertexIdToFaceIdAdjacency::FindAdjacentFaces(Edge.Verts[0], Edge.Verts[1], AdjFaces);
	}


	// Linearization of the EdgeToFaceMap
	TArray<SimpleEdge> EdgeArray;
	TArray<FaceList>   EdgeToFaces;   // index by edge: holds array of adj faces
	TArray<EdgeList>   VertexToEdges; // index by vertex: holds array of edges


private:
	// don't copy
	FAdjacencyData(const FAdjacencyData& other);
};

//This assumes the the number of Faces = NumIndices / 3
void ProxyLOD::SplitHardAngles(const float HardAngleRadians, const TArray<FVector3f>& FaceNormals, const int32 NumVerts, TArray<uint32>& Indices, std::vector<uint32_t>& AdditionalVertices)
{

	const uint32 NumIndices = Indices.Num();
	// Number of faces of the mesh.  This assumes triangles only!

	const uint32 NumFaces = NumIndices / 3;

	checkSlow(NumFaces == FaceNormals.Num());

	// Basic Adjacency Data

	FAdjacencyData Adjacency(Indices.GetData(), NumIndices, NumVerts);

	// Edge Count

	const int32 NumEdges = Adjacency.EdgeArray.Num();

	// Empty the duplicate vertex array.  Note using std::vector<uint32_t> to allow us to share some code with the UV system which also adds / splits verts.

	std::vector<uint32_t>().swap(AdditionalVertices);

	// Compute the angle, in radians, for each edge.  If an edge is adjacent to 
	// more than two faces, we set this angle to zero.

	TArray<float> EdgeAngleArray;
	ResizeArray(EdgeAngleArray, NumEdges);
	{

		// Compute the difference between faces normals at each edge.  
		// Make this zero if only one face is adjacent to edge.
		// Make this zero if more than 2 faces are adjacent to edge.

		for (int32 edgeIdx = 0; edgeIdx < NumEdges; ++edgeIdx)
		{
			// The adjacent faces to this edge
			const auto& Faces = Adjacency.EdgeToFaces[edgeIdx];

			if (Faces.Num() == 2)
			{
				const FVector3f& N0 = FaceNormals[Faces[0]];
				const FVector3f& N1 = FaceNormals[Faces[1]];

				const float CosOfAngle = FMath::Clamp(FVector3f::DotProduct(N0, N1), -1.f, 1.f);
				const float AngleInRadians = FMath::Acos(CosOfAngle); // Note this is in Radians in the range [0:Pi]

				EdgeAngleArray[edgeIdx] = AngleInRadians;
			}
			else
			{
				EdgeAngleArray[edgeIdx] = 0.f;
			}

		}
	}

	// Construct a list of unique verts that need to be split.
	// NB: multiple "Hard" edges could connect to a single vert.

	TArray<int32> SplitVertexList;
	{

		// Create a mask of valid edges (those that are adjacent to two faces)
		// NB: could parallelize
		TArray<int16> TwoFaceEdgeMask;
		ResizeInializedArray(TwoFaceEdgeMask, NumEdges);

		for (int32 edgeIdx = 0; edgeIdx < NumEdges; ++edgeIdx)
		{
			TwoFaceEdgeMask[edgeIdx] = 1;
			const auto& Faces = Adjacency.EdgeToFaces[edgeIdx];
			checkSlow(Faces.Num() > 0); 
			if (Faces.Num() != 2)
			{
				TwoFaceEdgeMask[edgeIdx] = 0;
			}
		}


		// Loop over the edges, finding the ones that exceed the hard angle limit
		// and marking the associated vertexs.
		// VertexSplitMask[i] = 1  if the vert should be split.


		TArray<int16> VertToSplitMask;
		ResizeInializedArray(VertToSplitMask, NumVerts);

		for (int32 vertIdx = 0; vertIdx < NumVerts; ++vertIdx) VertToSplitMask[vertIdx] = 0;

		for (int32 edgeIdx = 0; edgeIdx < NumEdges; ++edgeIdx)
		{
			// ignore edges that don't have exactly two faces

			if (TwoFaceEdgeMask[edgeIdx] == 0)
			{
				continue;
			}

			// ignore edges that are under the threshold

			if (EdgeAngleArray[edgeIdx] < HardAngleRadians)
			{
				continue;
			}

			// The edge in question

			const FAdjacencyData::SimpleEdge& Edge = Adjacency.EdgeArray[edgeIdx];

			// Mark the verts of this hard edge  
			// NB: a vert may be shared by multiple hard edges,
			// but that is fine.

			VertToSplitMask[Edge.Verts[0]] = 1;
			VertToSplitMask[Edge.Verts[1]] = 1;

		}

		// Insure that all the edges that are adjacent to a split vert candidate
		// have two faces.  
		// @todo: relax this requirement.

		// Mask out any vertex that has an "invalid" edge.
		// @todo Parallelize
		for (int32 vertIdx = 0; vertIdx < NumVerts; ++vertIdx)
		{
			if (VertToSplitMask[vertIdx] == 1)
			{
				bool bValid = true;

				// Get all adjacent edges and test that they are all valid.

				const auto& AdjEdges = Adjacency.VertexToEdges[vertIdx];

				for (auto edgeIdx : AdjEdges)
				{
					if (TwoFaceEdgeMask[edgeIdx] == 0)
					{
						bValid = false;
					}
				}

				if (bValid == false)
				{
					VertToSplitMask[vertIdx] = 0;
				}
			}
		}

		// Count the number of verts to split

		int32 NumSplitVerts = 0;
		for (int32 vertIdx = 0; vertIdx < NumVerts; ++vertIdx)
		{
			NumSplitVerts += VertToSplitMask[vertIdx];
		}

		// Populate the list of verts to split.
		{
			ResizeInializedArray(SplitVertexList, NumSplitVerts);
		}

		int32 offset = 0;
		for (int32 vertIdx = 0; vertIdx < NumVerts; ++vertIdx)
		{
			if (VertToSplitMask[vertIdx] == 1)
			{
				checkSlow(offset < NumSplitVerts); 
				SplitVertexList[offset] = vertIdx;
				offset++;
			}
		}

	}

	// return if there is actually no work to be done.
	// NB: the AdditionalVertices have already been emptied.

	if (SplitVertexList.Num() == 0)
	{
		return;
	}



	// Now that the verts have been identified, they could
	// processed independently. 

	// For each split vertex, build a list of different face groups.
	// A FaceGroup is an array of faceIds that should share a single
	// vertex (after splitting).
	typedef FAdjacencyData::FaceList    FaceGroupType;
	typedef TArray<FaceGroupType>       ListOfFaceGroupType;

	TArray<ListOfFaceGroupType> PerVertArrayOfFaceGroups;
	ResizeInializedArray(PerVertArrayOfFaceGroups, SplitVertexList.Num());


	// NB: this could be done in parallel.
	Parallel_For(FIntRange(0, SplitVertexList.Num()), [&](const FIntRange& Range)->void
	{
		//	for (int32 i = 0, IMax = SplitVertexList.Num(); i < IMax; ++i)
		for (int32 i = Range.begin(); i < Range.end(); ++i)
		{
			// The index of the split vert in the vertex array - of the split verts, this is the "i-th" one.

			const int32 splitVertIdx = SplitVertexList[i];

			checkSlow(splitVertIdx < NumVerts && splitVertIdx > -1); 

			// 
			// -- Need to establish connectivity between the faces that are adjacent to the split vert.
	        //

		    // All the edges that are adjacent to this vert.

			const auto& VertexAdjacentEdges = Adjacency.VertexToEdges[splitVertIdx];

			// All the faces that are adjacent to this vertex.

			const auto& AdjFaces = Adjacency.VertexToFaces[splitVertIdx];
			const int32 NumAdjFaces = AdjFaces.Num();

			// Start grouping the faces with their neighbor by constructing something
			// like a link-list.

			// Generate the link-list elements : each one "owns" a faceId
			TArray<FAdjacencyData::FaceAssociation> FaceToFaceAssociation;
			for (const auto& faceIdx : AdjFaces)
			{
				FaceToFaceAssociation.Add(FAdjacencyData::FaceAssociation(faceIdx));
			}

			// A map to index into the link-list by faceId
			std::map<int32, FAdjacencyData::FaceAssociation*> AssociationMap;
			for (int32 a = 0, AMax = (int32)FaceToFaceAssociation.Num(); a < AMax; ++a)
			{
				FAdjacencyData::FaceAssociation& Association = FaceToFaceAssociation[a];
				AssociationMap[Association.FaceId] = &(Association);
			}

			// loop over the edges, making associations between adjacent faces if the edge isn't "Sharp"
			// Keep track of the sharpest, non-hard edge.  This runner-up may be used to help split the faces
			// should there be only one hard edge.

			float SharpestAbsAngle = -1.f;
			int32 SharpestEdgeIdx = -1;
			for (const auto& edgeIdx : VertexAdjacentEdges)
			{
				const float AbsCurrentAngle = EdgeAngleArray[edgeIdx];

				if (AbsCurrentAngle < HardAngleRadians) // Not a "Hard" edge.  The faces should be connected in this case.
				{
					// Keep track of the sharpest non-hard edge.  Will have to use this to form the splitting groups
					// if there aren't any "hard edges" leaving this vert.

					if (AbsCurrentAngle > SharpestAbsAngle)
					{
						SharpestAbsAngle = AbsCurrentAngle;
						SharpestEdgeIdx = edgeIdx;
					}

					// The faces adj to this edge

					const auto& Faces = Adjacency.EdgeToFaces[edgeIdx];
					// This is redundant check.  We have already required that all edges have two faces.
					checkSlow(Faces.Num() < 3); // need this to be manifold!

												// By convention, for our link-list LastId <= FaceId <=NextId
					if (Faces.Num() == 2)
					{
						int32 FaceA, FaceB;
						if (Faces[0] < Faces[1])
						{
							FaceA = Faces[0];
							FaceB = Faces[1];
						}
						else
						{
							FaceA = Faces[1];
							FaceB = Faces[0];
						}

						checkSlow(FaceA != FaceB); 

						auto& AssociationA = AssociationMap[FaceA];
						AssociationA->NextId = FaceB;

						auto& AssociationB = AssociationMap[FaceB];
						AssociationB->LastId = FaceA;
					}

				}
			}

			// How many groups do our associations define?
			// Lets count the number of times LastId == FaceId;
			int32 GroupCount = 0;
			int32 LastCount = 0;
			int32 NextCount = 0;
			for (const auto& Association : FaceToFaceAssociation)
			{
				if (Association.LastId == Association.FaceId)
				{
					LastCount++;
					GroupCount++;
				}
				if (Association.NextId == Association.FaceId)
				{
					NextCount++;
				}
			}
			
			checkSlow(LastCount > 0 && NextCount > 0);

			// If we have only one group, then use the next sharpest edge to break it into two
			// if possible.

			if (GroupCount == 1 && SharpestEdgeIdx != -1)
			{
				// Get the faces for the next sharpest edge
				const auto& Faces = Adjacency.EdgeToFaces[SharpestEdgeIdx];

				checkSlow(Faces.Num() < 3); // need this to be manifold!

				if (Faces.Num() == 2)
				{
					int32 FaceA, FaceB;
					if (Faces[0] < Faces[1])
					{
						FaceA = Faces[0];
						FaceB = Faces[1];
					}
					else
					{
						FaceA = Faces[1];
						FaceB = Faces[0];
					}

					checkSlow(FaceA != FaceB); 

					auto& AssociationA = AssociationMap[FaceA];
					//checkSlow(AssociationA->NextId == FaceB || AssociationA->NextId == FaceA || AssociationA->LastId == FaceA);
					AssociationA->NextId = FaceA;

					auto& AssociationB = AssociationMap[FaceB];
					//checkSlow(AssociationB->LastId == FaceA || AssociationB->NextId == FaceB || AssociationB->LastId == FaceB);
					AssociationB->LastId = FaceB;

					GroupCount++;
				}
			}

			// Should I fix the triangles now?  Okay, lets do that, but it might be better to store this information and do that 
			// in a following parallel pass.

			// loop over the groups in the association.
			//  The i-th split vert now has the face group

			ListOfFaceGroupType& FaceGroups = PerVertArrayOfFaceGroups[i];
			if (GroupCount > 1)
			{
				for (auto& Association : FaceToFaceAssociation)
				{
					if (Association.FaceId != -1)
					{
						// Add this group.
						FaceGroups.Add(FaceGroupType());

						FaceGroupType& FacesInGroup = FaceGroups.Last();
						FacesInGroup.Add(Association.FaceId);

						// Go Forward, if there is a next
						if (Association.NextId != Association.FaceId)
						{
							int32 NextId = Association.NextId;
							int32 CurId = Association.FaceId;
							while (NextId != CurId && CurId != -1)
							{
								auto& CurAssociation = AssociationMap[NextId];
								CurId = CurAssociation->FaceId;
								NextId = CurAssociation->NextId;
								if (CurId != -1) FacesInGroup.Add(CurId);

								// Mark as used
								CurAssociation->FaceId = -1;
							}
						}

						// Go backward, if there is a Last
						if (Association.LastId != Association.FaceId)
						{
							int32 LastId = Association.LastId;
							int32 CurId = Association.FaceId;
							while (LastId != CurId && CurId != -1)
							{
								auto& CurAssociation = AssociationMap[LastId];
								CurId = CurAssociation->FaceId;
								LastId = CurAssociation->LastId;
								if (CurId != -1) FacesInGroup.Add(CurId);

								// Mark as used
								CurAssociation->FaceId = -1;
							}
						}

						// Mark this one as used:
						Association.FaceId = -1;
					}
				}
			}
			else  // There was only one group.  Put all the faces in it.
			{
				FaceGroups.Add(FaceGroupType());
				FaceGroupType& FacesInGroup = FaceGroups.Last();
				for (auto& Association : FaceToFaceAssociation)
				{
					FacesInGroup.Add(Association.FaceId);
				}
			}

		}
	});
	// Loop over the verts to split and use the face groups to re-write the trianlges
	// As the same time capture the AdditionalVertices.
	// NB: this would have to be re-worked a little if you wanted to parallelize this.

	for (int32 i = 0, IMax = SplitVertexList.Num(); i < IMax; ++i)
	{
		// Get the index of the i-th split vert
		const int32 splitVertIdx = SplitVertexList[i];

		// Get the face groups of the i-th split vert
		const ListOfFaceGroupType& FaceGroups = PerVertArrayOfFaceGroups[i];

		// New VertexIdx


		for (int32 faceGroupIDx = 0, FGmax = (int32)FaceGroups.Num(); faceGroupIDx < FGmax; ++faceGroupIDx)
		{
			// Allow the first group to use the pre-exsiting vertex
			if (faceGroupIDx == 0) continue;

			const auto& FacesInGroup = FaceGroups[faceGroupIDx];

			// Where the new vert will live.
			uint32 NewVertOffset = (uint32)AdditionalVertices.size() + (uint32)NumVerts;

			for (const auto FaceId : FacesInGroup)
			{

				// for each face
				// loop over the vertIDs for this face,
				// and re-wire the one that should point to
				// the new vertex.
				int32 offset = 3 * FaceId;
				for (int32 v = 0; v < 3; ++v)
				{
					checkSlow(v + offset < (int32)NumIndices);

					if (Indices[v + offset] == splitVertIdx)
					{
						Indices[v + offset] = NewVertOffset;
					}
				}

			}
			// Keep track of the verts we need to copy.
			AdditionalVertices.push_back(splitVertIdx);
		}


	} // end loop over split verts.

}

void ProxyLOD::SplitHardAngles(const float HardAngleDegrees, FVertexDataMesh& InOutMesh)
{
	const float HardAngleRadians = FMath::Abs(FMath::DegreesToRadians(HardAngleDegrees));

	// Number of Indices and Faces

	const int32 NumIndices = InOutMesh.Indices.Num();
	const int32 NumTriangles = NumIndices / 3;

	if (NumTriangles < 2) return;

	// Allocate space for the face normals

	TArray<FVector3f>  FaceNormals;
	ResizeArray(FaceNormals, NumTriangles);

	// Compute face normals in parallel

	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumTriangles), [&InOutMesh, &FaceNormals](const ProxyLOD::FIntRange& Range)
	{
		const uint32* Indices = InOutMesh.Indices.GetData();

		FVector3f Pos[3];
		for (uint32 f = Range.begin(), F = Range.end(); f < F; ++f)
		{
			const uint32 offset = 3 * f;
			const uint32 ids[3] = { Indices[offset] , Indices[offset + 1] , Indices[offset + 2] };
			Pos[0] = InOutMesh.Points[ids[0]];
			Pos[1] = InOutMesh.Points[ids[1]];
			Pos[2] = InOutMesh.Points[ids[2]];

			FaceNormals[f] = ComputeNormal(Pos);

		}
	});

	const int32 NumVerts = InOutMesh.Points.Num();
	std::vector<uint32_t> dupVerts;
	ProxyLOD::SplitHardAngles(HardAngleRadians, FaceNormals, NumVerts, InOutMesh.Indices, dupVerts);


	// add the duplicated verts, copying all the associated data.

	SplitVertices(InOutMesh, dupVerts);

}

namespace
{
	// Use the duplication vector extend the InOutVector with the correct values.
	template <typename T>
	void RemapMeshData(TArray<T>& InOutVector, int32 OldSize, const std::vector<uint32_t>& dupList)
	{
		uint32 NumDup = dupList.size();
		if (InOutVector.Num() == OldSize && NumDup != 0)
		{

			TArray<T> Tmp;
			ResizeArray(Tmp, OldSize + NumDup);

			for (int32 i = 0; i < OldSize; ++i)
			{
				Tmp[i] = InOutVector[i];
			}

			for (uint32 d = 0; d < NumDup; ++d)
			{
				Tmp[OldSize + d] = InOutVector[dupList[d]];
			}

			Swap(Tmp, InOutVector);
		}
	}
}



void ProxyLOD::SplitVertices(FVertexDataMesh& InOutMesh, const std::vector<uint32_t>& dupVerts)
{
	const uint32 NumDup = dupVerts.size();

	// Early out.
	if (NumDup == 0) return;

	const uint32 OldVertNum = InOutMesh.Points.Num();

	RemapMeshData(InOutMesh.Points, OldVertNum, dupVerts);

	RemapMeshData(InOutMesh.Normal, OldVertNum, dupVerts);

	RemapMeshData(InOutMesh.Tangent, OldVertNum, dupVerts);

	RemapMeshData(InOutMesh.BiTangent, OldVertNum, dupVerts);

	RemapMeshData(InOutMesh.TransferNormal, OldVertNum, dupVerts);

	RemapMeshData(InOutMesh.TangentHanded, OldVertNum, dupVerts);

	RemapMeshData(InOutMesh.UVs, OldVertNum, dupVerts);

	RemapMeshData(InOutMesh.FaceColors, OldVertNum, dupVerts);

	RemapMeshData(InOutMesh.FacePartition, OldVertNum, dupVerts);

}


void ProxyLOD::CacheNormals(FVertexDataMesh& InMesh)
{
	const int32 NumNormals = InMesh.Normal.Num();
	TArray<FVector3f>& CachedNormals = InMesh.TransferNormal;
	ResizeArray(CachedNormals, NumNormals);

	for (int32 i = 0; i < NumNormals; ++i)
	{
		CachedNormals[i] = InMesh.Normal[i];
	}
}

// Testing function using for debugging.  Verifies that if A is adjacent to B, then B is adjacent to A.
// Returns the number of failure cases ( should be zero!)
int ProxyLOD::VerifyAdjacency(const uint32* EdgeAdjacentFaceArray, const uint32 NumEdgeAdjacentFaces)
{
	int FailureCount = 0;
	const uint32 NumFaces = NumEdgeAdjacentFaces / 3;
	check(NumEdgeAdjacentFaces % 3 == 0);

	// loop over faces.  Each three enteries hold the Ids of the adjacent faces.

	for (uint32 f = 0; f < NumFaces; ++f)
	{
		// offset to face 'f'
		const uint32 FaceIdx = f * 3;

		// The three faces adjacent to face 'f'
		const uint32 AdjFaces[3] = { EdgeAdjacentFaceArray[FaceIdx], EdgeAdjacentFaceArray[FaceIdx + 1], EdgeAdjacentFaceArray[FaceIdx + 2] };

		for (int i = 0; i < 3; ++i)
		{

			bool bValid = AdjFaces[i] < NumFaces;

			// offset to the 'i'-th face adjacent to face 'f' - call it fprime

			const uint32 AdjFaceIdx = 3 * AdjFaces[i];

			// faces adjacent to fprime.
			const uint32 AdjAdjFaces[3] = { EdgeAdjacentFaceArray[AdjFaceIdx], EdgeAdjacentFaceArray[AdjFaceIdx + 1], EdgeAdjacentFaceArray[AdjFaceIdx + 2] };

			// one of these should be 'f' itself.
			bValid = bValid && (AdjAdjFaces[0] == f || AdjAdjFaces[1] == f || AdjAdjFaces[2] == f);

			if (!bValid)
			{
				FailureCount += 1;
			}
		}
	}

	return FailureCount;
}
// Face Averaged Normals. 
void ProxyLOD::ComputeFaceAveragedVertexNormals(FAOSMesh& InOutMesh)
{

	// Generate adjacency data
	FVertexIdToFaceIdAdjacency AdjacencyData(InOutMesh.Indexes, InOutMesh.GetNumIndexes(), InOutMesh.GetNumVertexes());
	
	uint32 NumFaces = InOutMesh.GetNumIndexes() / 3;

	// Generate face normals.
	FVector3f* FaceNormals = new FVector3f[NumFaces];

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumFaces), [&InOutMesh, FaceNormals](const ProxyLOD::FUIntRange& Range)
	{
		FVector3f Pos[3];
		for (uint32 f = Range.begin(), F = Range.end(); f < F; ++f)
		{
			const openvdb::Vec3I Tri = InOutMesh.GetFace(f);
			Pos[0] = InOutMesh.Vertexes[Tri[0]].GetPos();
			Pos[1] = InOutMesh.Vertexes[Tri[1]].GetPos();
			Pos[2] = InOutMesh.Vertexes[Tri[2]].GetPos();

			FaceNormals[f] = ComputeNormal(Pos);

		}
	});


	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, InOutMesh.GetNumVertexes()), [&AdjacencyData, &InOutMesh, FaceNormals](const ProxyLOD::FUIntRange&  Range)
	{
		// loop over vertexes in this range
		for (uint32 v = Range.begin(), V = Range.end(); v < V; ++v)
		{
			// This vertex
			auto& AOSVertex = InOutMesh.Vertexes[v];

			// zero the associated normal
			AOSVertex.Normal = FVector3f(0.f, 0.f, 0.f);

			// loop over all the faces that share this vertex, accumulating the normal
			const auto& AdjFaces = AdjacencyData.VertexToFaces[v];
			checkSlow(AdjFaces.Num() != 0);

			if (AdjFaces.Num() != 0)
			{
				for (auto FaceId: AdjFaces)
				{
					checkSlow(FaceId > -1);
					AOSVertex.Normal += FaceNormals[FaceId];
				}
				AOSVertex.Normal.Normalize();
			}
		}
	});

	// clean up
	delete[] FaceNormals;
}


void ProxyLOD::AddDefaultTangentSpace(FVertexDataMesh& VertexDataMesh)
{

	// Copy the tangent space attributes.

	const uint32 DstNumPositions = VertexDataMesh.Points.Num();

	TArray<FVector3f>& NormalArray = VertexDataMesh.Normal;
	TArray<FVector3f>& TangetArray = VertexDataMesh.Tangent;
	TArray<FVector3f>& BiTangetArray = VertexDataMesh.BiTangent;

	// Allocate space
	ProxyLOD::FTaskGroup TaskGroup;


	TaskGroup.Run([&]
	{
		ResizeArray(NormalArray, DstNumPositions);
	});

	TaskGroup.Run([&]
	{
		ResizeArray(TangetArray, DstNumPositions);
	});

	TaskGroup.RunAndWait([&]
	{
		ResizeArray(BiTangetArray, DstNumPositions);
	});



	// Transfer the normal
	{
		ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, DstNumPositions),
			[&NormalArray](const ProxyLOD::FUIntRange& Range)
		{
			for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				NormalArray[i] = FVector3f(0, 0, 1);
			}
		}
		);
	}

	// Add default values for tangents and BiTangent for testing
	{
		ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, DstNumPositions),
			[&TangetArray](const ProxyLOD::FUIntRange& Range)
		{
			for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				TangetArray[i] = FVector3f(1, 0, 0);
			}
		}
		);
	}
	{
		ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, DstNumPositions),
			[&BiTangetArray](const ProxyLOD::FUIntRange& Range)
		{
			for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				BiTangetArray[i] = FVector3f(0, 1, 0);
			}
		}
		);
	}
}

void ProxyLOD::ComputeBogusTangentAndBiTangent(FVertexDataMesh& VertexDataMesh)
{
	const int32 NumVerts = VertexDataMesh.Points.Num();
	// Lets go ahead and add a BS tangent space.
	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumVerts),
		[&VertexDataMesh](const ProxyLOD::FIntRange&  Range)
	{
		auto& TangentArray = VertexDataMesh.Tangent;
		auto& BiTangentArray = VertexDataMesh.BiTangent;
		auto& NormalArray = VertexDataMesh.Normal;

		for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
		{

			TangentArray[i] = FVector3f(1, 0, 0);
			BiTangentArray[i] = FVector3f::CrossProduct(NormalArray[i], TangentArray[i]);
			TangentArray[i] = FVector3f::CrossProduct(BiTangentArray[i], NormalArray[i]);

		}

	});

}

void ProxyLOD::ComputeBogusNormalTangentAndBiTangent(FVertexDataMesh& VertexDataMesh)
{
	const int32 NumVerts = VertexDataMesh.Points.Num();
	auto& TangentArray = VertexDataMesh.Tangent;
	auto& BiTangentArray = VertexDataMesh.BiTangent;
	auto& NormalArray = VertexDataMesh.Normal;
	auto& H = VertexDataMesh.TangentHanded;

	ResizeArray(TangentArray, NumVerts);
	ResizeArray(BiTangentArray, NumVerts);
	ResizeArray(NormalArray, NumVerts);
	ResizeArray(H, NumVerts);


	// Lets go ahead and add a BS tangent space.
	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumVerts),
		[&](const ProxyLOD::FIntRange&  Range)
	{
		for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
		{

			TangentArray[i] = FVector3f(1, 0, 0);
			BiTangentArray[i] = FVector3f(0, 1, 0);
			NormalArray[i] = FVector3f(0, 0, 1);
			H[i] = 1;
		}

	});

}


template <typename T>
static void TAddNormals(TAOSMesh<T>& Mesh)
{
	ProxyLOD::ComputeFaceAveragedVertexNormals(Mesh);
}

template<>
void TAddNormals<FPositionOnlyVertex>(TAOSMesh<FPositionOnlyVertex>& Mesh)
{
	// Doesn't support normals.
}


void ProxyLOD::AddNormals(TAOSMesh<FPositionOnlyVertex>& InOutMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::AddNormals)

	TAddNormals(InOutMesh);
}



	/**
	* Attempt to correct the collapsed walls.  
	* NB: The kDOP tree is already built, using the same mesh.
	*
	* @param Indices  - mesh conectivity
	* @param Positions - vertex locations: maybe changed by this function.
	* @param VoxelSize - length scale used in heuristic that determins how far to move vertices.
	*/
	int32 CorrectCollapsedWalls( const ProxyLOD::FkDOPTree& kDOPTree, 
		                         const TArray<uint32>& IndexArray, 
		                         TArray<FVector3f>& PositionArray,
		                         const float VoxelSize)
	{
		typedef uint32 EdgeIdType;
		typedef uint32 FaceIdType;
		

		ProxyLOD::FUnitTransformDataProvider kDOPDataProvider(kDOPTree);

		// Number of edges in our mesh

		int32 NumEdges = IndexArray.Num();

		// This will hold the intersecting faces for each edge.

		std::unordered_map<FaceIdType, std::vector<FaceIdType>> IntersectionListMap;

		const auto* Indices = IndexArray.GetData();
		auto* Pos           = PositionArray.GetData();

		// loop over the polys and collect the names of the faces that intersect.
		auto GetFace = [Indices, Pos](int32 FaceIdx, FVector3f(&Verts)[3])
		{
			const uint32 Idx[3] = { Indices[3 * FaceIdx], Indices[3 * FaceIdx + 1], Indices[3 * FaceIdx + 2] };

			Verts[0] = Pos[Idx[0]];
			Verts[1] = Pos[Idx[1]];
			Verts[2] = Pos[Idx[2]];
		};

		auto GetFaceNormal = [&GetFace](int32 FaceIdx)->FVector3f
		{
			FVector3f Verts[3];
			GetFace(FaceIdx, Verts);

			return ComputeNormal(Verts);
		};

		int32 TestCount = 0;
		for (int32 FaceIdx = 0; FaceIdx < NumEdges / 3; ++FaceIdx)
		{

			FVector3f Verts[3];
			GetFace(FaceIdx, Verts);

			const FVector3f FaceNormal = ComputeNormal(Verts);

			// loop over these three edges.
			for (int32 j = 0; j < 3; ++j)
			{
				int32 sV = j;
				int32 eV = (j + 1) % 3;

				FkHitResult kDOPResult;

				TkDOPLineCollisionCheck<const ProxyLOD::FUnitTransformDataProvider, uint32>  EdgeRay(FVector(Verts[sV]), FVector(Verts[eV]), true, kDOPDataProvider, &kDOPResult);

				bool bHit = kDOPTree.LineCheck(EdgeRay);

				if (bHit)
				{
					// Triangle we hit
					int32 HitTriId = kDOPResult.Item;

					// Don't count a hit against myself
					if (HitTriId == FaceIdx)
					{
						continue;
					}

					// Make sure the hit wasn't just one of the verts.
					if (kDOPResult.Time > 0.999 || kDOPResult.Time < 0.001)
					{
						continue;
					}

					// We only care about faces pointing in opposing directions
					const FVector3f HitFaceNormal = GetFaceNormal(HitTriId);
					if (FVector3f::DotProduct(FaceNormal, HitFaceNormal) > -0.94f) // not in 160 to 200 degrees
					{
						continue;
					}

					TestCount++;

					auto Search = IntersectionListMap.find(FaceIdx);
					if (Search != IntersectionListMap.end())
					{
						auto& FaceList = Search->second;
						if (std::find(FaceList.begin(), FaceList.end(), HitTriId) == FaceList.end())
						{
							FaceList.push_back(HitTriId);
						}
					}
					else
					{
						std::vector<FaceIdType> FaceList;
						FaceList.push_back(HitTriId);
						IntersectionListMap[FaceIdx] = FaceList;
					}
				}
			}
		}

		//For each triangle that collides, push it a small fixed distance in the normal direction.
		{
			for (auto ListMapIter = IntersectionListMap.begin(); ListMapIter != IntersectionListMap.end(); ++ListMapIter)
			{
				int32 FaceIdx = ListMapIter->first;
				const FVector3f TriNormal = GetFaceNormal(FaceIdx);

				// Scale by a small amount

				const FVector3f NormDisplacement = TriNormal * (VoxelSize / 7.f);
				const uint32 Idx[3] = { Indices[3 * FaceIdx], Indices[3 * FaceIdx + 1], Indices[3 * FaceIdx + 2] };

				Pos[Idx[0]] += NormDisplacement;
				Pos[Idx[1]] += NormDisplacement;
				Pos[Idx[2]] += NormDisplacement;

			}

		}
		return TestCount;
	}

/**
* Attempt to correct the collapsed walls.
* NB: The kDOP tree is already built, using the same mesh.
*
* @param Indices  - mesh conectivity
* @param Positions - vertex locations: maybe changed by this function.
* @param VoxelSize - length scale used in heuristic that determins how far to move vertices.
*/
int32 CorrectCollapsedWalls(const ProxyLOD::FkDOPTree& kDOPTree,
	FMeshDescription& MeshDescription,
	const float VoxelSize)
{
	typedef uint32 EdgeIdType;
	typedef uint32 FaceIdType;
	TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();

	ProxyLOD::FUnitTransformDataProvider kDOPDataProvider(kDOPTree);

	// Number of triangle in our mesh
	int32 NumTriangle = MeshDescription.Triangles().Num();

	// This will hold the intersecting faces for each edge.

	std::unordered_map<FaceIdType, std::vector<FaceIdType>> IntersectionListMap;

	//const auto* Indices = IndexArray.GetData();
	//auto* Pos = PositionArray.GetData();

	// loop over the polys and collect the names of the faces that intersect.
	auto GetFace = [&MeshDescription, &VertexPositions](int32 FaceIdx, FVector3f(&Verts)[3])
	{
		const FVertexID Idx[3] = { MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(3 * FaceIdx)),
								MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(3 * FaceIdx + 1)),
								MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(3 * FaceIdx + 2)) };

		Verts[0] = VertexPositions[Idx[0]];
		Verts[1] = VertexPositions[Idx[1]];
		Verts[2] = VertexPositions[Idx[2]];
	};

	auto GetFaceNormal = [&GetFace](int32 FaceIdx)->FVector3f
	{
		FVector3f Verts[3];
		GetFace(FaceIdx, Verts);

		return ComputeNormal(Verts);
	};

	int32 TestCount = 0;
	for (int32 FaceIdx = 0; FaceIdx < NumTriangle; ++FaceIdx)
	{
		FVector3f Verts[3];
		GetFace(FaceIdx, Verts);

		const FVector3f FaceNormal = ComputeNormal(Verts);

		// loop over these three edges.
		for (int32 j = 0; j < 3; ++j)
		{
			int32 sV = j;
			int32 eV = (j + 1) % 3;

			FkHitResult kDOPResult;

			TkDOPLineCollisionCheck<const ProxyLOD::FUnitTransformDataProvider, uint32>  EdgeRay(FVector(Verts[sV]), FVector(Verts[eV]), true, kDOPDataProvider, &kDOPResult);

			bool bHit = kDOPTree.LineCheck(EdgeRay);

			if (bHit)
			{
				// Triangle we hit
				int32 HitTriId = kDOPResult.Item;

				// Don't count a hit against myself
				if (HitTriId == FaceIdx)
				{
					continue;
				}

				// Make sure the hit wasn't just one of the verts.
				if (kDOPResult.Time > 0.999 || kDOPResult.Time < 0.001)
				{
					continue;
				}

				// We only care about faces pointing in opposing directions
				const FVector3f HitFaceNormal = GetFaceNormal(HitTriId);
				if (FVector3f::DotProduct(FaceNormal, HitFaceNormal) > -0.94f) // not in 160 to 200 degrees
				{
					continue;
				}

				TestCount++;

				auto Search = IntersectionListMap.find(FaceIdx);
				if (Search != IntersectionListMap.end())
				{
					auto& FaceList = Search->second;
					if (std::find(FaceList.begin(), FaceList.end(), HitTriId) == FaceList.end())
					{
						FaceList.push_back(HitTriId);
					}
				}
				else
				{
					std::vector<FaceIdType> FaceList;
					FaceList.push_back(HitTriId);
					IntersectionListMap[FaceIdx] = FaceList;
				}
			}
		}
	}

	//For each triangle that collides, push it a small fixed distance in the normal direction.
	{
		for (auto ListMapIter = IntersectionListMap.begin(); ListMapIter != IntersectionListMap.end(); ++ListMapIter)
		{
			int32 FaceIdx = ListMapIter->first;
			const FVector3f TriNormal = GetFaceNormal(FaceIdx);

			// Scale by a small amount

			const FVector3f NormDisplacement = TriNormal * (VoxelSize / 7.f);

			const FVertexID Idx[3] = {	MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(3 * FaceIdx)),
										MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(3 * FaceIdx + 1)),
										MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(3 * FaceIdx + 2)) };

			VertexPositions[Idx[0]] += NormDisplacement;
			VertexPositions[Idx[1]] += NormDisplacement;
			VertexPositions[Idx[2]] += NormDisplacement;
		}

	}
	return TestCount;
}

int32 ProxyLOD::CorrectCollapsedWalls(FMeshDescription& InOutMeshDescription, const float VoxelSize)
{
	// Build an acceleration structure

	FkDOPTree kDOPTree;
	BuildkDOPTree(InOutMeshDescription, kDOPTree);

	return CorrectCollapsedWalls(kDOPTree, InOutMeshDescription, VoxelSize);
}

int32 ProxyLOD::CorrectCollapsedWalls(FVertexDataMesh& InOutMesh,
	const float VoxelSize)
{

	// Build an acceleration structure

	FkDOPTree kDOPTree;
	BuildkDOPTree(InOutMesh, kDOPTree);

	return CorrectCollapsedWalls(kDOPTree, InOutMesh.Indices, InOutMesh.Points, VoxelSize);
}

FBoxSphereBounds ProxyLOD::GetBounds(const FVertexDataMesh& InMesh)
{
	FBoxSphereBounds3f Bounds(InMesh.Points.GetData(), InMesh.Points.Num());
	return FBoxSphereBounds(Bounds);
}

double ProxyLOD::GetWorldSpaceArea(const FVertexDataMesh& InMesh)
{
	const uint32* Indices = InMesh.Indices.GetData();
	const FVector3f* Points = InMesh.Points.GetData();
	const int32 NumTriangles = InMesh.Indices.Num() / 3;

	double Mesh3DArea = 0;

	for (int32 IdxTri = 0; IdxTri < NumTriangles; ++IdxTri)
	{
		const uint32 Offset = 3 * IdxTri;
		Mesh3DArea += UE::Geometry::VectorUtil::Area(Points[Indices[Offset + 0]], Points[Indices[Offset + 1]], Points[Indices[Offset + 2]]);
	}

	return Mesh3DArea;
}

// Test to insure that no two vertexes share the same position.
void ProxyLOD::TestUniqueVertexes(const FMixedPolyMesh& InMesh)
{
	const uint32 NumVertexes = InMesh.Points.size();
	const openvdb::Vec3s* Vertexes = &InMesh.Points[0];

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumVertexes),
		[&NumVertexes, &Vertexes](const ProxyLOD::FUIntRange& Range)
	{
		for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
		{
			const auto& VertexI = Vertexes[i];
			for (uint32 j = i + 1; j < NumVertexes; ++j)
			{
				const auto& VertexJ = Vertexes[j];
				checkSlow(VertexI != VertexJ);
			}
		}
	});

}


// Test to insure that no two vertexes share the same position.
void ProxyLOD::TestUniqueVertexes(const FAOSMesh& InMesh)
{
	const uint32 NumVertexes = InMesh.GetNumVertexes();
	const FPositionNormalVertex* Vertexes = InMesh.Vertexes;

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumVertexes),
		[&NumVertexes, &Vertexes](const ProxyLOD::FUIntRange& Range)
	{
		for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
		{
			const FPositionNormalVertex& VertexI = Vertexes[i];
			for (uint32 j = i + 1; j < NumVertexes; ++j)
			{
				const FPositionNormalVertex& VertexJ = Vertexes[j];
				checkSlow(VertexI.GetPos() != VertexJ.GetPos());
				checkSlow(false == VertexI.GetPos().Equals(VertexJ.GetPos(), 1.e-6f));
			}
		}
	});

}


void ProxyLOD::ColorPartitions(FMeshDescription& InOutRawMesh, const std::vector<uint32>& partitionResults)
{

	// testing - coloring the simplified mesh by the partitions generated by uvatlas
	FColor Range[13] = { FColor(255, 0, 0), FColor(0, 255, 0), FColor(0, 0, 255), FColor(255, 255, 0), FColor(0, 255, 255),
		FColor(153, 102, 0), FColor(249, 129, 162), FColor(29, 143, 177), FColor(118, 42, 145),
		FColor(255, 121, 75), FColor(102, 204, 51), FColor(153, 153, 255), FColor(255, 255, 255) };

	TArrayView<FVector4f> VertexInstanceColors = FStaticMeshAttributes(InOutRawMesh).GetVertexInstanceColors().GetRawArray();

	//Remap the vertex instance
	int32 TriangleIndex = 0;
	TMap<uint32, FVertexInstanceID> WedgeIndexToVertexInstanceID;
	WedgeIndexToVertexInstanceID.Reserve(InOutRawMesh.VertexInstances().Num());
	for (const FTriangleID TriangleID : InOutRawMesh.Triangles().GetElementIDs())
	{
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			WedgeIndexToVertexInstanceID.Add((TriangleIndex * 3) + Corner, InOutRawMesh.GetTriangleVertexInstance(TriangleID, Corner));
		}
		TriangleIndex++;
	}

	for (int i = 0; i < partitionResults.size(); ++i)
	{
		uint32 PId = partitionResults[i];
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			FVertexInstanceID VertexInstanceID = WedgeIndexToVertexInstanceID[(i * 3) + Corner];
			VertexInstanceColors[VertexInstanceID] = FLinearColor(Range[PId % 13]);
		}
	}
}


// Add Face colors to the a mesh according the the partitionResults array.

void ProxyLOD::ColorPartitions(FVertexDataMesh& InOutMesh, const std::vector<uint32>& PartitionResults)
{

	// testing - coloring the simplified mesh by the partitions generated by uvatlas
	FColor Range[13] = { FColor(255, 0, 0), FColor(0, 255, 0), FColor(0, 0, 255), FColor(255, 255, 0), FColor(0, 255, 255),
		FColor(153, 102, 0), FColor(249, 129, 162), FColor(29, 143, 177), FColor(118, 42, 145),
		FColor(255, 121, 75), FColor(102, 204, 51), FColor(153, 153, 255), FColor(255, 255, 255) };

	const uint32 NumFaces = InOutMesh.Indices.Num() / 3;
	ResizeArray(InOutMesh.FaceColors, NumFaces);

	for (int i = 0; i < PartitionResults.size(); ++i)
	{
		uint32 PId = PartitionResults[i];
		InOutMesh.FaceColors[i] = Range[PId % 13];
	}
}


void ProxyLOD::AddWedgeColors(FMeshDescription& RawMesh)
{

	FColor ColorRange[13] = { FColor(255, 0, 0), FColor(0, 255, 0), FColor(0, 0, 255), FColor(255, 255, 0), FColor(0, 255, 255),
		FColor(153, 102, 0), FColor(249, 129, 162), FColor(29, 143, 177), FColor(118, 42, 145),
		FColor(255, 121, 75), FColor(102, 204, 51), FColor(153, 153, 255), FColor(255, 255, 255) };

	TArrayView<FVector4f> VertexInstanceColors = FStaticMeshAttributes(RawMesh).GetVertexInstanceColors().GetRawArray();

	//Recolor the vertex instances
	int32 TriangleIndex = 0;
	for (const FTriangleID TriangleID : RawMesh.Triangles().GetElementIDs())
	{
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			VertexInstanceColors[RawMesh.GetTriangleVertexInstance(TriangleID, Corner)] = FLinearColor(ColorRange[((TriangleIndex*3) + Corner) % 13]);
		}
		TriangleIndex++;
	}
}




template <typename T>
static void TMakeCube(TAOSMesh<T>& Mesh, float Length)
{


	// The 8 corners of unit cube

	FVector3f Pos[8];

	Pos[0] = FVector3f(0, 0, 1);
	Pos[1] = FVector3f(1, 0, 1);
	Pos[2] = FVector3f(1, 0, 0);
	Pos[3] = FVector3f(0, 0, 0);

	Pos[4] = FVector3f(0, 1, 1);
	Pos[5] = FVector3f(1, 1, 1);
	Pos[6] = FVector3f(1, 1, 0);
	Pos[7] = FVector3f(0, 1, 0);


	const uint32 IndexList[36] = {
		/* front */
		0, 1, 2,
		2, 3, 0,
		/* right */
		2, 1, 5,
		5, 6, 2,
		/* back */
		5, 4, 7,
		7, 6, 5,
		/*left*/
		7, 4, 0,
		0, 3, 7,
		/*top*/
		0, 4, 5,
		5, 1, 0,
		/*bottom*/
		7, 3, 2,
		2, 6, 7
	};

	// Scale to Length size cube

	for (int32 i = 0; i < 8; ++i) Pos[i] *= Length;


	// Create the mesh

	const uint32 NumVerts = 8;
	const uint32 NumTris = 12; // two per cube face

	Mesh.Resize(NumVerts, NumTris);

	// copy the indices into the mesh

	for (int32 i = 0; i < NumTris * 3; ++i) Mesh.Indexes[i] = IndexList[i];

	// copy the locations into the mesh

	for (int32 i = 0; i < NumVerts; ++i) Mesh.Vertexes[i].GetPos() = Pos[i];

	TAddNormals(Mesh);

}

void ProxyLOD::MakeCube(FAOSMesh& InOutMesh, float Length)
{
	TMakeCube(InOutMesh, Length);
}

void ProxyLOD::MakeCube(TAOSMesh<FPositionOnlyVertex>& InOutMesh, float Length)
{
	TMakeCube(InOutMesh, Length);
}

void ProxyLOD::AddNormals(FAOSMesh& InOutMesh)
{
	TAddNormals(InOutMesh);
}

// Unused
#if 0
// Computes the face normals and assigns them to the wedges TangentZ
static void ComputeRawMeshNormals(FMeshDescription& InOutMesh)
{
	const int32 NumFaces = InOutMesh.WedgeIndices.Num() / 3;

	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumFaces),
		[&InOutMesh](const ProxyLOD::FIntRange& Range)
	{
		auto& Normal = InOutMesh.WedgeTangentZ;
		auto& Pos = InOutMesh.VertexPositions;
		auto& WedgeToPos = InOutMesh.WedgeIndices;
		// loop over these faces
		uint32  WIdxs[3];
		FVector3f Verts[3];
		for (uint32 f = Range.begin(), F = Range.end(); f < F; ++f)
		{
			// get the three corners for this face
			WIdxs[0] = f * 3;
			WIdxs[1] = WIdxs[0] + 1;
			WIdxs[2] = WIdxs[0] + 2;

			Verts[0] = Pos[WedgeToPos[WIdxs[0]]];
			Verts[1] = Pos[WedgeToPos[WIdxs[1]]];
			Verts[2] = Pos[WedgeToPos[WIdxs[2]]];

			// Compute the face normal
			// NB: this assumes a counter clockwise orientation.
			FVector3f FaceNormal = ComputeNormal(Verts);

			// Assign this to all the corners (wedges)
			Normal[WIdxs[0]] = FaceNormal;
			Normal[WIdxs[1]] = FaceNormal;
			Normal[WIdxs[2]] = FaceNormal;

		}
	}
	);
}



static void ComputeAngleAveragedNormal(FVertexDataMesh& VertexDataMesh)
{

	
	const int32 NormalType = 1;
	if (NormalType == 0) return;

	// Generate adjacency data
	FAdjacencyData  AdjacencyData(VertexDataMesh.Indices.GetData(), VertexDataMesh.Indices.Num(), VertexDataMesh.Points.Num());


	const int32 NumFaces = VertexDataMesh.Indices.Num() / 3;

	// - Compute array of face normals.

	// FaceNormals 
	TArray<FVector3f> FaceNormalArray;
	FaceNormalArray.Empty(NumFaces);
	FaceNormalArray.AddUninitialized(NumFaces);

	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumFaces),
		[&FaceNormalArray, &VertexDataMesh](const ProxyLOD::FIntRange& Range)
	{
		const auto& Indices = VertexDataMesh.Indices;
		const auto& Pos = VertexDataMesh.Points;

		FVector3f Verts[3];
		for (int32 f = Range.begin(), F = Range.end(); f < F; ++f)
		{
			int32 Idx0 = f * 3;
			Verts[0] = Pos[Indices[Idx0 + 0]];
			Verts[1] = Pos[Indices[Idx0 + 1]];
			Verts[2] = Pos[Indices[Idx0 + 2]];

			FaceNormalArray[f] = ComputeNormal(Verts);
		}

	});

	const int32 NumVerts = VertexDataMesh.Points.Num();
	auto& VertexNormalArray = VertexDataMesh.Normal;

	ResizeArray(VertexNormalArray, NumVerts);

	// for each vertex, get the associated faces.
	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumVerts),
		[NormalType, &AdjacencyData, &VertexDataMesh, &FaceNormalArray, &VertexNormalArray](const ProxyLOD::FIntRange&  Range)
	{
		const auto& PositionArray = VertexDataMesh.Points;
		const auto& Indices = VertexDataMesh.Indices;
		// loop over vertexes in this range
		for (int32 v = Range.begin(), V = Range.end(); v < V; ++v)
		{
			// zero the associated normal
			VertexNormalArray[v] = FVector3f(0.f, 0.f, 0.f);

			// loop over all the faces that share this vertex, accumulating the normal
			const auto& AdjFaces = AdjacencyData.VertexAdjacentFaceArray[v];
			checkSlow(AdjFaces.size() != 0);

			if (AdjFaces.size() != 0)
			{
				for (auto FaceCIter = AdjFaces.cbegin(); FaceCIter != AdjFaces.cend(); ++FaceCIter)
				{
					int32 FaceIdx = *FaceCIter;

					if (NormalType != 1)
					{
						int32 Idx = FaceIdx * 3;

						FVector3f NextMinusCurrent;
						FVector3f PrevMinusCurrent;

						if (v == Indices[Idx + 0])
						{
							NextMinusCurrent = PositionArray[Indices[Idx + 1]] - PositionArray[v];
							PrevMinusCurrent = PositionArray[Indices[Idx + 2]] - PositionArray[v];

						}
						else if (v == Indices[Idx + 1])
						{
							NextMinusCurrent = PositionArray[Indices[Idx + 2]] - PositionArray[v];
							PrevMinusCurrent = PositionArray[Indices[Idx + 0]] - PositionArray[v];
						}
						else if (v == Indices[Idx + 2])
						{
							NextMinusCurrent = PositionArray[Indices[Idx + 0]] - PositionArray[v];
							PrevMinusCurrent = PositionArray[Indices[Idx + 1]] - PositionArray[v];
						}
						else
						{
							// Should never happen
							check(0);
						}

						NextMinusCurrent.Normalize();
						PrevMinusCurrent.Normalize();

						// compute the angle
						float CosAngle = FVector3f::DotProduct(NextMinusCurrent, PrevMinusCurrent);
						CosAngle = FMath::Clamp(CosAngle, -1.f, 1.f);
						const float Angle = FMath::Acos(CosAngle);

						VertexNormalArray[v] += FaceNormalArray[FaceIdx] * Angle;
					}
					else
					{
						VertexNormalArray[v] += FaceNormalArray[FaceIdx];
					}
				}

				VertexNormalArray[v].Normalize();
			}
		}
	});

	TArray<FVector3f> TestVertexNormalArray;
	ResizeArray(TestVertexNormalArray, NumVerts);

	for (int32 v = 0; v < NumVerts; ++v)  TestVertexNormalArray[v] = FVector3f(0, 0, 0);
	// Testing. Loop over all the verts and make sure they are mostly in the same direction as the face normals.
	for (int32 face = 0; face < NumFaces; ++face)
	{
		int32 offset = face * 3;

		const auto& faceNormal = FaceNormalArray[face];
		for (int32 i = offset; i < offset + 3; ++i)
		{
			uint32 v = VertexDataMesh.Indices[i];
			TestVertexNormalArray[v] += faceNormal;
		}
	}

	for (int32 v = 0; v < NumVerts; ++v)  TestVertexNormalArray[v].Normalize();

	for (int32 v = 0; v < NumVerts; ++v)
	{

		const auto& vertexNormal = VertexNormalArray[v];
		const auto& testVertexNormal = TestVertexNormalArray[v];

		checkSlow(FVector3f::DotProduct(vertexNormal, testVertexNormal) > 0.99);

	}

	// Testing. Loop over all the verts and make sure they are mostly in the same direction as the face normals.
	for (int32 face = 0; face < NumFaces; ++face)
	{
		int32 offset = face * 3;

		const auto& faceNormal = FaceNormalArray[face];
		for (int32 i = offset; i < offset + 3; ++i)
		{
			uint32 v = VertexDataMesh.Indices[i];
			const auto& vertexNormal = VertexNormalArray[v];

			//	checkSlow(FVector3f::DotProduct(vertexNormal, faceNormal) > 0.3);
		}
	}



	// Lets go ahead and add a BS tangent space.
	ProxyLOD::Parallel_For(ProxyLOD::FIntRange(0, NumVerts),
		[&VertexDataMesh](const ProxyLOD::FIntRange&  Range)
	{
		auto& TangentArray = VertexDataMesh.Tangent;
		auto& BiTangentArray = VertexDataMesh.BiTangent;
		const auto& NormalArray = VertexDataMesh.Normal;

		for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
		{
			const FVector3f& Normal = NormalArray[i];
			FVector3f Tangent(1, 0, 0);
			Tangent = Tangent - Normal * FVector3f::DotProduct(Normal, Tangent);
			Tangent.Normalize();

			FVector3f BiTangent(0, 1, 0);
			BiTangent = BiTangent - Normal * FVector3f::DotProduct(Normal, BiTangent);
			BiTangent = BiTangent - Tangent * FVector3f::DotProduct(Tangent, BiTangent);
			BiTangent.Normalize();

			TangentArray[i] = Tangent;
			BiTangentArray[i] = BiTangent;

		}

	});
}
#endif 


#undef LOCTEXT_NAMESPACE