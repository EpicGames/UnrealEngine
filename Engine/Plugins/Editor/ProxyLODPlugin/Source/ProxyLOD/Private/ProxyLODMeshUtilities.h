// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ProxyLODMeshTypes.h"
#include "ProxyLODThreadedWrappers.h"

THIRD_PARTY_INCLUDES_START
#include <vector>
THIRD_PARTY_INCLUDES_END

namespace ProxyLOD
{
	enum class ENormalComputationMethod : uint8
	{
		AreaWeighted,
		AngleWeighted,
		EqualWeighted
	};

	/**
	* Calls into the direxXMesh library to compute the per-vertex normal, by default this will weight by area.
	*
	* NB: this is different than computing on the raw mesh, which can result in per-index tangent space.
	* 
	* @param InOutMesh        Vertex-based mesh which will be updated by this function with with a per-vertex normal
	* @param Method           Determine if the normal is angle-weighted or area-weighted or equal weighted.
	*/
	void ComputeVertexNormals( FVertexDataMesh& InOutMesh, 
		                       const ENormalComputationMethod Method = ENormalComputationMethod::AngleWeighted);
	/**
	* Uses UE code in the Mesh Utilities module to compute the tangent space
	* The computation is a Mikk-T(angent) space version of tangent space.  
	*
	* NB: This updates the FMeshDescription::TangentX and FMeshDescription::TangentY and optionally TangentZ.
	*     This can result in a per-wedge tangent space if the normals are split.
	* 
	* @param InOutMesh          FMeshDescription which will be updated by this function - adding a tangent space
	* @param bRecomputeNormals  Determine if the normal should be recomputed as well (defualt - no).
	*/
	void ComputeTangentSpace(FMeshDescription& InOutMesh,
		                      const bool bRecomputeNormals = false );
	/**
	* Calls into the direxXMesh library to compute the per-vertex tangent and bitangent, optionally recomputes the normal.
	*
	* NB: this is different than computing on the raw mesh, which can result in per-index tangent space.
	*
	* @param InOutMesh          Vertex-based which will be updated by this function - adding a tangent space
	* @param bRecomputeNormals  Determine if the normal should be recomputed as well (defualt - no).
	*/
	void ComputeTangentSpace( FVertexDataMesh& InOutMesh, 
		                      const bool bRecomputeNormals = false );

	/**
	* Simple routine for adding face-averaged vertex normals to an Array of Structs vertex-based triangle mesh.
	*
	* @param InOutMesh  Vertex-based array of structs mesh, face averaged normals will be added.
	*/
	void ComputeFaceAveragedVertexNormals( FAOSMesh& InOutMesh );

	/**
	* Add a default tangent space that is aligned with the coordinate axes :(1, 0, 0), (0, 1, 0), (0, 0, 1) to a vertex data mesh.  
	* 
	* NB: This tangent space really has nothing to do with the actual geometry of the mesh!
	* 
	* @param InOutMesh  Vertex-based mesh to which we add the default tangent space.
	*/
	void AddDefaultTangentSpace( FVertexDataMesh& InOutMesh );


	/**
	* Simple routine for adding face-averaged vertex normals to an Array of Structs vertex-based triangle mesh.
	*
	* @param InOutMesh  Vertex-based array of structs mesh, face averaged normals will be added.
	*/
	void AddNormals( FAOSMesh& InOutMesh );
	void AddNormals( TAOSMesh<FPositionOnlyVertex>& InOutMesh);

	/**
	* Testing function using for debugging.  Verifies that if A is adjacent to B, then B is adjacent to A.
	* Returns the number of failure cases ( should be zero!)
	*
	* @param  EdgeAdjacentFaceArray  Each element in the array corresponds to the edge of a triangle and holds index of
	*                                the adjacent exterior face. 
	* @param  NumEdgeAdjacentFaces   Length of EdgeAdjacentFaceArray
	*/
	int VerifyAdjacency(const uint32* EdgeAdjacentFaceArray, const uint32 NumEdgeAdjacentFaces);

	/**
	* Splits vertices along hard edges (as defined by the difference in face normals).
	*
	* NB: When a vertex is split, all the data is copied to the duplicate vertices - including any tangent space data.
	* In general you will want to compute vertex normals and tangent space after doing this.
	*
	* @param HardAngleDegrees    The hard angle threshold, in degrees, used to determine if the vertices on an edge should be split.
	* @param InOutMesh           The mesh that may be altered by the
	*/
	void SplitHardAngles(const float HardDegrees, FVertexDataMesh& InOutMesh);

	/**
	* Alters the connectivity to split vertices along hard edges (as defined by the difference in face normals).
	*
	* @param HardAngleRadians    The hard angle threshold, in radians, used to determine if the vertices on an edge should be split.
	* @param FaceNormals         Pointer to array of face normals.  This is assumed to be of length NumIndices / 3.
	*
	* @param NumVerts            Total number of verts.  Usually this will be one more than the max of the Indices[] array.
	*
	* @param Indices             The index buffer.  Contents but not length maybe changed on return!
	*
	* @param AdditionalVertices  On return An array that maps old vertices to new ones
	*                            (i.e. need to add Vertex[NumVerts + i] as a copy of VertexId = AdditionVertices[i] )
	*/
	void SplitHardAngles(const float HardAngleRadians, const TArray<FVector3f>& FaceNormals, const int32 NumVerts, TArray<uint32>& Indices, std::vector<uint32_t>& AdditionalVertices);

	/**
	* Splits vertices in the InOutMesh according to the mapping provided by the dupVerts array
	* This assumes that the index buffer has already been updated to reflect the new vertices.
	*
	* @param InOutMesh           The mesh to be altered.
	* @param dupVerts            A mapping of new vertices to old.
	*
	* If N = Number of vertices on input, the resulting mesh will have N + dupVerts.size()  verts.
	* The vertices are duplicated with the following rule.
	*
	*   Vertex[N + d] = Vertex[dupVert[d]]
	*/
	void SplitVertices(FVertexDataMesh& InOutMesh, const std::vector<uint32_t>& dupVerts);

	/**
	* Copy the data in the member Normal Array to the memeber TransferNormal Array
	*/
	void CacheNormals(FVertexDataMesh& InMesh);

	/**
	* Very thin walls can develop mesh interpenetration (opposing wall surfaces meet) during simplification.  This
	* can produce rendering artifacts ( related to distance field shadows and ao).
	*
	* @param InOutMesh    Mesh that may have collapsed walls. This function attempts 
	*                     to slightly separate interpenetrating faces with opposing normals.
	* @param LengthScale  Used to in a heuristic to determine the distance to move interpenetrating verts.
	*
	* @return  Number of detected inter-penetrations of opposing faced polys.
	*/
	int32 CorrectCollapsedWalls( FMeshDescription& InOutMeshDescription,
								 const float LengthScale);
	int32 CorrectCollapsedWalls( FVertexDataMesh& InOutMesh,
		                         const float LengthScale );

	FBoxSphereBounds GetBounds(const FVertexDataMesh& InMesh);
	double GetWorldSpaceArea(const FVertexDataMesh& InMesh);
	
#if 0
	// Can be used to transfer things like vertex colors from source geometry to output.

	void TransferMeshAttributes( const FMeshDescriptionArrayAdapter& RawMeshArrayAdapter,
		                         const openvdb::Int32Grid& SrcPolyIndexGrid,
								 FMeshDescription& InOutMesh );
#endif 
	// --------------- debugging ---------------------------------------------------------------

	/**
	* Test to insure that no two vertexes share the same position.
	*
	* NB: Debugging function that does a checkslow.
	*
	* @param  InMesh  Mesh to test
	*/
	void TestUniqueVertexes(const FMixedPolyMesh& InMesh);
	void TestUniqueVertexes(const FAOSMesh& InMesh);

	/**
	* Add vertex (or wedge) colors according to the partition assigned to each face by the 
	* PartitionArray.  
	* 
	* NB: The partition array must have an entry for each face.  PartitionArray[FaceID] = PartitionId
	*
	* @param InOutMesh       Mesh to be colored.
	* @param PartitionArray  Maps each face to a partition.
	*/
	void ColorPartitions(FMeshDescription& InOutMesh,     const std::vector<uint32>& PartitionArray );
	void ColorPartitions( FVertexDataMesh& InMesh, const std::vector<uint32>& PartitionArray );

	
	/**
	* Simply color wedge colors according to index number. 
	* 
	* @param InOutMesh  Mesh to be colored.
	*/
	void AddWedgeColors(FMeshDescription& InOutMesh );
	
	/**
	* Add a default tangent and bi tangent  aligned with the coordinate axes :(1, 0, 0), (0, 1, 0) to a mesh.
	*
	* NB: This tangent space really has nothing to do with the actual geometry of the mesh!
	*
	* @param InOutMesh  Vertex-based mesh to which we add the tangent and bitangent
	*/
	void ComputeBogusTangentAndBiTangent(FVertexDataMesh& InOutMesh);

	/**
	* Add a default tangent space that is aligned with the coordinate axes :(1, 0, 0), (0, 1, 0), (0, 0, 1) to a vertex data mesh.
	*
	* NB: This tangent space really has nothing to do with the actual geometry of the mesh!
	*
	* @param InOutMesh  Vertex-based mesh to which we add the default tangent space.
	*/
	void ComputeBogusNormalTangentAndBiTangent(FVertexDataMesh& InOutMesh);

	/**
	* Make a cube, with side length "Length" and min/max corners at (0,0,0)  (L, L, L)
	*
	* @param OutMesh  On return holds the desired cube.
	* @param Length   The length of a side of the cube.
	*/
	void MakeCube(FAOSMesh& OutMesh, float Length);
	void MakeCube(TAOSMesh<FPositionOnlyVertex>& OutMesh, float Length);
	// --------------- end debugging ------------------------------------------------------------

}


