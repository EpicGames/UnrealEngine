// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODParallelSimplifier.h"
#include "ProxyLODMeshPartition.h"
#include "Async/ParallelFor.h"

void ProxyLOD::PartitionAndSimplifyMesh(const FBBox& SrcBBox, const float MinFractionToRetain, const float MaxFeatureCost, const int32 NumPartitions,
	FAOSMesh& InOutMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::PartitionAndSimplifyMesh)

	typedef FAOSMesh                           SimplifierMeshType;
	typedef ProxyLOD::FQuadricMeshSimplifier   FMeshSimplifier;

	check(NumPartitions > 0);

	// Create individual meshes for each partition.

	TArray<SimplifierMeshType> PartitionedMeshArray;
	PartitionedMeshArray.Empty(NumPartitions);
	PartitionedMeshArray.AddZeroed(NumPartitions);

	// Partition the mesh along the major axis.  This copies the InOutMesh into the PartitionedMeshArray 

	PartitionOnMajorAxis(InOutMesh, SrcBBox, NumPartitions, PartitionedMeshArray);

	// Empty the source mesh.

	InOutMesh.Empty();

	// Allocate space for the seam vert ids.

	TArray<int32>*  SeamVertexIdArrayCollection = new TArray<int32>[NumPartitions];

	// Create tasks for each partition.
	ParallelFor(
		NumPartitions,
		[&PartitionedMeshArray, &SeamVertexIdArrayCollection, &MinFractionToRetain, &MaxFeatureCost](int32 Index)
		{
			SimplifierMeshType& LocalMesh = PartitionedMeshArray[Index];
			TArray<int32>* SeamVertexIdArray = &SeamVertexIdArrayCollection[Index];
			const int32 MinTriNumToRetain = FMath::CeilToInt((LocalMesh.GetNumIndexes() / 3) * MinFractionToRetain);

			ProxyLOD::FSimplifierTerminatorBase Terminator(MinTriNumToRetain, MaxFeatureCost);
			SimplifyMesh(Terminator, LocalMesh, SeamVertexIdArray);
		},
		EParallelForFlags::Unbalanced
	);

	// Merge the partitioned mesh back into a single mesh.
	MergeMeshArray(PartitionedMeshArray, SeamVertexIdArrayCollection, InOutMesh);

	// Delete the array of seam verts ids
	if (SeamVertexIdArrayCollection)
	{
		delete[] SeamVertexIdArrayCollection;
	}
}

void ProxyLOD::ParallelSimplifyMesh(const FClosestPolyField& SrcReference, const float MinFractionToRetain, const float MaxFeatureCost, FAOSMesh& InOutMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::ParallelSimplifyMesh)

	// Nothing to do if we must retain 100% of the input
	if (MinFractionToRetain >= 1.0f || InOutMesh.GetNumIndexes() <= 3)
	{
		return;
	}

	const FMeshDescriptionArrayAdapter& SrcMeshAdapter = SrcReference.MeshAdapter();
	const FBBox  SrcBBox = SrcMeshAdapter.GetBBox();
	const size_t AOSTriNum = InOutMesh.GetNumIndexes() / 3;
	const int32  AOSTriNumToRetain = FMath::CeilToInt(SrcMeshAdapter.polygonCount() * MinFractionToRetain);
	const float  AOSMinFractionToRetain = float(AOSTriNumToRetain) / AOSTriNum;

	// Nothing to do if we must retain 100% of the AOS Mesh
	if (AOSMinFractionToRetain >= 1.0f)
	{
		return;
	}

	// Usage of prime numbers is to ensure seams will never be at the same places throughout the sequence
	// Changing those numbers will affect the output, so choose numbers high enough now
	// to take advantage of CPUs with many cores and properly balance tasks on them.
	TArray<int32> NumPartitionsSequence = { 31, 7 };

	// Remove partition size that do not make sense for number of tris available
	NumPartitionsSequence.RemoveAll([AOSTriNum](int32 Partitions) { return AOSTriNum < Partitions*10000; });

	// Each step will get rid of the same ratio until we reach the target for the final step
	// This provides good performance and quality results by leaving enough for subsequent steps to reduce.
	// For this, we want FractionToRetainPerStep ^ Steps = AOSMinFractionToRetain
	const float FractionToRetainPerStep = FMath::Exp(FMath::Loge(AOSMinFractionToRetain) / (NumPartitionsSequence.Num()+1));

	for (int32 NumPartitions : NumPartitionsSequence)
	{
		// Each partition will retain the specified ratio, which will be
		// proportional to the amount of tris it contains.
		// Low poly partition will remove less tri than high poly partitions providing a good balance.
		// Each partition is still subject to MaxFeatureCost so some partition might stop removing tris
		// once they are too low on poly.
		const FMajorAxisPartitionFunctor PartitionFunctor(SrcBBox, NumPartitions);
		PartitionAndSimplifyMesh(SrcBBox, FractionToRetainPerStep, MaxFeatureCost, NumPartitions, InOutMesh);
	}

	const ProxyLOD::FSimplifierTerminatorBase Terminator(AOSTriNumToRetain, MaxFeatureCost);
	SimplifyMesh(Terminator, InOutMesh);
}
