// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProxyLODMeshTypes.h"
#include "ProxyLODSimplifier.h"
#include "ProxyLODMeshUtilities.h"



namespace ProxyLOD
{


	/**
	* Replaces the input mesh with a simplified mesh.  
	* Uses edge-collapse quadric-error based mesh simplification.
	* The stopping criterion for mesh simplification is controlled by the Terminator
	*
	*  struct TerminatorInterface
	* {
	*    // Returns true if the simplification should terminate.
	*    // numTris - current number in the mesh
	*    // error   - quadric error of the next edge to collapse.  
	*    bool operator()(const int32 numTris, float error);
	* }
	* 
	* @param  Terminator            Provides a stopping criterion for the simplification.
	* @param  InOutMesh             The mesh to simplify and on return the simplified mesh.
	* @param  OutSeamVertexArray    Optional pointer to array that will capture Ids of the 
	*                               vertices in the simplified mesh that are on mesh boundaries.
	* @return  the max square error of an edge collapsed.
	*/
	template <typename TerminationCriterionType>
	static float SimplifyMesh( const TerminationCriterionType& Terminator, 
		                       FAOSMesh& InOutMesh,
		                       TArray<int32>* OutSeamVertexArray = NULL);


	/**
	* Replaces the input mesh with a simplified mesh.
	* Uses edge-collapse quadric-error based mesh simplification.
	* The stopping criterion for mesh simplification derived from the MinFractionToRetain
	* and MaxFeatureCost.
	*
	*  ( 1 ) Partitions the mesh into NumPartitions, locks the vertices (and edges) on the partition boundaries
	*        and simplifies.
	*  ( 2 ) Merges the simplified meshes back to a single mesh.
	*
	* @param  SrcBBox               Spatial region that will be partitioned into NumPartitions along the major axis.
	* @param  MinFractionToRetain   Fraction of original geometry to retain. Used in stopping criterion.
	* @param  MaxFeatureCost        Limit to the error allowed when simplifying.  Used in stopping criterion.
	* @param  InOutMesh             The mesh to simplify and on return the simplified mesh.
	*
	*/
	void PartitionAndSimplifyMesh( const FBBox& SrcBBox,
		                           const float MinFractionToRetain,
		                           const float MaxFeatureCost,
		                           const int32 NumPartitions,
		                           FAOSMesh& InOutMesh);

	/**
	* Replaces the input mesh with a simplified mesh.
	* Uses edge-collapse quadric-error based mesh simplification.
	* The stopping criterion for mesh simplification derived from the MinFractionToRetain
	* and MaxFeatureCost.
	*
	* This uses a fixed parallelism strategy with four waves of simplification.
	* each retaining a smaller percentage of polys.  The majority of the work is done
	* by first (higher) partition wave with subsequent waves cleaning up the seams 
	* that resulted from previous waves. 
	* 
	* @param  SrcPolyField          Container that holds the original (pre-voxelization / iso-surface extraction)
	*                               used to help determine the target number of triangles for the simplifier.
	* @param  MinFractionToRetain   Fraction of original geometry to retain. Used in stopping criterion.
	* @param  MaxFeatureCost        Limit to the error allowed when simplifying.  Used in stopping criterion.
	* @param  InOutMesh             The mesh to simplify and on return the simplified mesh.
	*
	*/
	void ParallelSimplifyMesh( const FClosestPolyField& SrcPolyField, 
		                       const float MinFractionToRatain, 
		                       const float MaxFeatureCost, 
		                       FAOSMesh& InOutMesh);
}

// --- Templated code implementation ---

namespace
{
	/**
	* Trait used to encode the defined limit of co-aligned faces when simplifing.
	* A collapse that would causes the distance between two adjacent faces normals to be more than the CoAlignment
	* will be penalized. 
	*
	* NB: This is currently not being used, and the MeshSimplyElements.h has a hard-coded limit of 90-degrees.
	*     The TSimpTri<T>::ReplaceVertexIsValid() API needs to be extended to take the critical dot-product value.
	*/
	template <typename VertexType>
	struct TCoAlignment
	{};

	template<>
	struct TCoAlignment<FPositionNormalVertex>
	{
		constexpr static float Value = 0.f;  // 90 degrees
	};



	// Code used in testing the feature cost of simplifying an already perfect cube of a given size.
	template <typename VertexType>
	static float CannonicalFeatureCost(float LenghtScale)
	{
		TAOSMesh<VertexType>  Cube;
		MakeCube(Cube, LenghtScale);
		int32 SrcNumTris = Cube.GetNumIndexes() / 3;
		float MaxFeatureCost = SimplifyMesh(SrcNumTris - 2, SrcNumTris, FLT_MAX, Cube);

		return MaxFeatureCost;
	}
}

template <typename TerminationCriterionType>
static inline float ProxyLOD::SimplifyMesh(const TerminationCriterionType& Terminator, FAOSMesh& InOutMesh, TArray<int32>* SeamVertexArray)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::SimplifyMesh)

	typedef FAOSMesh                          SimplifierMeshType;
	typedef ProxyLOD::FQuadricMeshSimplifier  FMeshSimplifier;

	// Lock the seams and capture them after simplification if requested

	const bool bLockAndCaptureSeams = (SeamVertexArray != NULL);

	// NB: these are ignored if the vertex type doesn't have normals.
	const float NormalWeights[] = { 1.f, 1.f, 1.f };

	float CoAlignmentLimit = TCoAlignment<MeshVertType>::Value;

	if (SeamVertexArray) SeamVertexArray->Empty();

	// No work to do on an empty mesh.

	if (InOutMesh.IsEmpty()) return 0.f;


	FMeshSimplifier Simplifier(InOutMesh.Vertexes, InOutMesh.GetNumVertexes(), InOutMesh.Indexes, InOutMesh.GetNumIndexes(), CoAlignmentLimit);

	// The Simplifier constructor did a deep copy of the mesh.  Free some memory.
	InOutMesh.Empty();

	if (bLockAndCaptureSeams)
	{
		Simplifier.SetBoundaryLocked();
	}

	Simplifier.SetAttributeWeights(NormalWeights);

	Simplifier.InitCosts();

	// Do the simplification.  
	const float MaxSqrError = Simplifier.SimplifyMesh(Terminator);

	// Resize the AOSMesh to hold the result - this empties it first.
	InOutMesh.Resize(Simplifier.GetNumVerts(), Simplifier.GetNumTris());

	// Copy the new mesh back into the AOSMeshedVolume and capture the seam vertexes.
	int NumVertices;
	int NumIndices;
	Simplifier.OutputMesh(InOutMesh.Vertexes, InOutMesh.Indexes, SeamVertexArray, &NumVertices, &NumIndices);

	// Set the number of vertices again after calling OutputMesh which removes duplicates
	InOutMesh.SetVertexAndIndexCount(NumVertices, NumIndices);

	return MaxSqrError;
}
