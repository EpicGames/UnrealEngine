// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODParameterization.h"

#include "ProxyLODMeshConvertUtils.h"
#include "ProxyLODMeshUtilities.h"
#include "ProxyLODMeshParameterization.h"
#include "StaticMeshAttributes.h"




class FProxyLODParameterizationImpl : public IProxyLODParameterization
{
public:
	FProxyLODParameterizationImpl() {}

	virtual ~FProxyLODParameterizationImpl() {};


	virtual bool ParameterizeMeshDescription(FMeshDescription& MeshDescription, int32 Width, int32 Height, float GutterSpace, float Stretch, int32 ChartNum, bool bUseNormals, bool bRecomputeTangentSpace, bool bPrintDebugMessages) const override
	{
		const float MaxStretch = Stretch;
		const size_t MaxChartNumber = ChartNum;
		const bool bUseNormalsInUV = bUseNormals;
		
		// No Op
		auto NoOpCallBack = [](float percent)->HRESULT {return  S_OK; };

		bool bComputeNormals = bRecomputeTangentSpace;
		bool bSplitHardAngles = false;
		bool bComputeTangentSpace = bRecomputeTangentSpace;

		float HardAngleDegrees = 80.f;
		FVertexDataMesh VertexDataMesh;
		ProxyLOD::ConvertMesh(MeshDescription, VertexDataMesh);

		if (bSplitHardAngles)
		{
			ProxyLOD::SplitHardAngles(HardAngleDegrees, VertexDataMesh);
		}

		if (bComputeNormals)
		{
			ProxyLOD::ComputeVertexNormals(VertexDataMesh, ProxyLOD::ENormalComputationMethod::AngleWeighted);
		}


		FIntPoint UVSize(Height, Width);
		ProxyLOD::FTextureAtlasDesc TextureAtlasDesc(UVSize, GutterSpace);

		float MaxStretchUsed = 0.f;
		size_t NumChartsMade = 0;
		bool bUVGenerationSucess = ProxyLOD::GenerateUVs(VertexDataMesh, TextureAtlasDesc, false, MaxStretch, MaxChartNumber, bUseNormalsInUV, NoOpCallBack, &MaxStretchUsed, &NumChartsMade);

		if (bUVGenerationSucess)
		{
			if (bPrintDebugMessages)
			{
				UE_LOGF(LogTemp, Warning, "UV Generation: generated %zu charts, using max stretch  %f ", NumChartsMade, MaxStretchUsed);
			}

			if (bComputeTangentSpace)
			{
				ProxyLOD::ComputeTangentSpace(VertexDataMesh, false);
			}
			// Reset the result mesh description
			MeshDescription.Empty();
			FStaticMeshAttributes(MeshDescription).Register();

			// convert to fill the mesh description
			ProxyLOD::ConvertMesh(VertexDataMesh, MeshDescription);
		}


		return bUVGenerationSucess;
	};


	virtual bool GenerateUVs(int32                    Width,
		                     int32                    Height,
		                     float                    GutterSpace,
			                 const TArray<FVector3f>&   VertexBuffer,
			                 const TArray<int32>&     IndexBuffer,
			                 const TArray<int32>&     AdjacencyBuffer,
			                 TFunction<bool(float)>&  Callback,
			                 TArray<FVector2D>&       UVVertexBuffer,
			                 TArray<int32>&           UVIndexBuffer,
			                 TArray<int32>&           VertexRemapArray,
			                 float&                   MaxStretch,
			                 int32&                   NumCharts) const 
	{
		ProxyLOD::FTextureAtlasDesc TextureAtlasDesc(FIntPoint(Height, Width), GutterSpace);
		return ProxyLOD::GenerateUVs(TextureAtlasDesc, VertexBuffer, IndexBuffer, AdjacencyBuffer, Callback, UVVertexBuffer, UVIndexBuffer, VertexRemapArray, MaxStretch, NumCharts);
	}
};



TUniquePtr<IProxyLODParameterization> IProxyLODParameterization::CreateTool()
{
	TUniquePtr<IProxyLODParameterization> Tool = MakeUnique<FProxyLODParameterizationImpl>();

	if (Tool == nullptr)
	{
		return nullptr;
	}

	return Tool;

}
