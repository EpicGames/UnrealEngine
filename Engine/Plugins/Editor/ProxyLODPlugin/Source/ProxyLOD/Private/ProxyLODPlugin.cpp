// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "IProxyLODPlugin.h"
#include "MeshMergeData.h"
#include "MaterialUtilities.h" // for FFlattenMaterial 
#include "Engine/StaticMesh.h"
#include "Misc/ScopedSlowTask.h"
#include "Stats/StatsMisc.h"
#include "Trace/Trace.h"
#include "HAL/RunnableThread.h"

#define PROXYLOD_CLOCKWISE_TRIANGLES  1

#define LOCTEXT_NAMESPACE "ProxyLODMeshReduction"

#include "ProxyLODMaterialTransferUtilities.h"
#include "ProxyLODMeshAttrTransfer.h"
#include "ProxyLODMeshConvertUtils.h"
#include "ProxyLODMeshTypes.h"
#include "ProxyLODMeshUtilities.h"
#include "ProxyLODMeshParameterization.h"
#include "ProxyLODMeshSDFConversions.h"

#include "ProxyLODParallelSimplifier.h"
#include "ProxyLODRasterizer.h"
#include "ProxyLODSimplifier.h"

#include "ProxyLODOpenVDB.h"

#include "ProxyLODkDOPInterface.h"
#include "ProxyLODThreadedWrappers.h"

#include "StaticMeshAttributes.h"



#define PROXYLOD_USE_TEST_CVARS 1

#if PROXYLOD_USE_TEST_CVARS	

// Disable any mesh simplification and UV-generation.
// Will result in very high poly red geometry.
static TAutoConsoleVariable<int32> CVarProxyLODRemeshOnly(
	TEXT("r.ProxyLODRemeshOnly"),
	0,
	TEXT("Only remesh.  No simplification or materials. Default off.\n")
	TEXT("0: Disabled - will simplify and generate materials \n")
	TEXT("1: Enabled  - will not simplfy or generate materials."),
	ECVF_Default);

// Allow for adding vertex colors to the output mesh that correspond
// to the different charts in the uv-atlas.
static TAutoConsoleVariable<int32> CVarProxyLODChartColorVerts(
	TEXT("r.ProxyLODChartColorVerts"),
	0,
	TEXT("Color verts by uv chart.  Default off.\n")
	TEXT("0: Disabled \n")
	TEXT("1: Enabled."),
	ECVF_Default);

// Testing methods to choose correct hit when shooting rays between
// simplified and source goemetry.
static TAutoConsoleVariable<int32> CVarProxyLODTransfer(
	TEXT("r.ProxyLODTransfer"),
	1,
	TEXT("0: shoot both ways\n")
	TEXT("1: preference for forward (default)"),
	ECVF_Default);


// Attempt to separate the sides of collapsed walls.
static TAutoConsoleVariable<int32> CVarProxyLODCorrectCollapsedWalls(
	TEXT("r.ProxyLODCorrectCollapsedWalls"),
	0,
	TEXT("Shall the ProxyLOD system attemp to correct walls with interpenetrating faces")
	TEXT("0: disabled (default)\n")
	TEXT("1: endable, may cause cracks."),
	ECVF_Default);


// Testing tangent space encoding of normal maps by optionally using
// a default wold-space aligned tangent space.
static TAutoConsoleVariable<int32> CVarProxyLODUseTangentSpace(
	TEXT("r.ProxyLODUseTangentSpace"),
	1,
	TEXT("Shall the ProxyLOD system generate a true tangent space at each vertex")
	TEXT("0: world space at each vertex\n")
	TEXT("1: tangent space at each vertex (default)"),
	ECVF_Default);


// Force the simplifier to use single threaded code path.
static TAutoConsoleVariable<int32> CVarProxyLODSingleThreadSimplify(
	TEXT("r.ProxyLODSingleThreadSimplify"),
	0,
	TEXT("Use single threaded code path. Default off.\n")
	TEXT("0: Multithreaded \n")
	TEXT("1: Single threaded."),
	ECVF_Default);

// Disable the parallel flattening of the source textures.
static TAutoConsoleVariable<int32> CVarProxyLODMaterialInParallel(
	TEXT("r.ProxyLODMaterialInParallel"),
	1,
	TEXT("0: disable doing material work in parallel with mesh simplification\n")
	TEXT("1: enable - default"),
	ECVF_Default);

// Limit the number of dilation steps used in gap filling.
static TAutoConsoleVariable<int32> CVarProxyLODMaxDilationSteps(
	TEXT("r.ProxyLODMaxDilationSteps"),
	7,
	TEXT("Limit the numer of dilation steps used in gap filling for performance reasons\n")
	TEXT("This may affect gap filling quality as bigger dilations steps will be used with a smaller max \n")
	TEXT("0: will disable gap filling\n")
	TEXT("7: default\n"),
	ECVF_Default);

#endif

/**
* Implementation of the required IMeshMerging Interface.
* This class does the actual work.
*
* See HieararchicalLOD.cpp
*/
class FVoxelizeMeshMerging : public IMeshMerging
{
public:

	typedef TArray<FMeshMergeData>			FMeshMergeDataArray;
	typedef TArray<FInstancedMeshMergeData> FInstancedMeshMergeDataArray;
	typedef TArray<FFlattenMaterial>		FFlattenMaterialArray;
	typedef openvdb::math::Transform		OpenVDBTransform;

	FVoxelizeMeshMerging();

	~FVoxelizeMeshMerging();

	// Construct the proxy geometry and materials - the results
	// are captured by a call back.

	virtual void ProxyLOD(const FMeshMergeDataArray& InData,
		                  const FMeshProxySettings& InProxySettings,
		                  const FFlattenMaterialArray& InputMaterials,
		                  const FGuid InJobGUID) override;

	virtual void ProxyLOD(const FInstancedMeshMergeDataArray& InData,
		const FMeshProxySettings& InProxySettings,
		const FFlattenMaterialArray& InputMaterials,
		const FGuid InJobGUID) override;

	virtual void AggregateLOD() override
	{};

	virtual bool bSupportsParallelMaterialBake() override;
	

	virtual FString GetName() override
	{
		return FString("ProxyLODMeshMerging");
	}

	//  Update internal options from current CVar values.
	void CaptureCVars();


private:
	void ProxyLOD(FMeshDescriptionArrayAdapter& InSrcGeometryAdapter, FMeshDescriptionArrayAdapter& InClippingGeometryAdapter, const FMeshProxySettings& InProxySettings, const FFlattenMaterialArray& InputMaterials, const FGuid InJobGUID);

	// Restore default parameters
	void RestoreDefaultParameters();

	// Compute the voxel resolution and create XForm from world to voxel space
	// @param InProxySetting  Control setting from UI
	// @param ObjectSize      Major Axis length of the geometry
	OpenVDBTransform::Ptr ComputeResolution(const FMeshProxySettings& InProxySettings, float ObjectSize);

private:

	// In voxel units.  This defines the isosurface to be extracted
	// from the levelset.

	float IsoSurfaceValue = 0.5;

	// Determine the method used to determine the correct ray hit
	// when creating a correspondence between simplified geometry
	// and src geometry.

	int32 RayHitOrder = 1;

	// Used in gap-closing.  This max is to bound a potentially expensive
	// computation.  If the gap size requires more dilation steps at the current voxel
	// size, then the dilation (and erosion) will be done with larger voxels. 

	int32 MaxDilationSteps = 7;

	// Flag to set if the verts should be colored by char in the UV atlas.
	// Useful for debuging.

	bool bChartColorVerts = false;

	// Flag to set that determines if a true tangent space is generated.
	// if false, a (1,0,0), (0, 1,0), (0, 0,1) space is used on each vert.

	bool bUseTangentSpace = true;

	// Flag to disable the simplification and transfer of materials.
	// When set to true the input material will be voxelized and remeshed
	// only.

	bool bRemeshOnly = false;

	// Flag that can enable/disable the multi-threaded aspect of the simplifier.

	bool bMultiThreadSimplify = true;

	// Flag to enable the thin wall correction.
	// Very thin walls can develop mesh interpenetration(opposing wall surfaces meet) during simplification.This
	// can produce rendering artifacts(related to distance field shadows and ao).

	bool bCorrectCollapsedWalls = false;
};

/**
*  Required MeshReduction Interface.
*/
class FProxyLODMeshReduction : public IProxyLODMeshReduction
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;


	/** IMeshReductionModule interface.*/
	// not supported !
	virtual class IMeshReduction* GetSkeletalMeshReductionInterface() override;
	
	// not supported !
	virtual class IMeshReduction* GetStaticMeshReductionInterface()   override;

	/** 
	* Voxel-based merging & remeshing.  
	* @return pointer to a member data instance of specialized IMeshMerging subclass
	*/
	virtual class IMeshMerging*   GetMeshMergingInterface()           override;

	/**
	* Retrieve the distributed mesh merging interface.
	* NB: currently not supported
	* @todo: add support.
	*/
	virtual class IMeshMerging* GetDistributedMeshMergingInterface()  override;
	

	virtual FString GetName() override
	{
		return FString("ProxyLODMeshReduction");
	};

private:

	FVoxelizeMeshMerging MeshMergingTool;
};

DEFINE_LOG_CATEGORY_STATIC(LogProxyLODMeshReduction, Log, All);

IMPLEMENT_MODULE(FProxyLODMeshReduction, ProxyLODMeshReduction)

// Reuse FRunnableThread facilities to support FTlsAutoCleanup on TBB threads
class FTBBThread : private FRunnableThread
{
	virtual void SetThreadPriority( EThreadPriority ) override {}
	virtual void Suspend( bool ) override {}
	virtual bool Kill( bool ) override { return false; }
	virtual void WaitForCompletion() override {}
	virtual bool CreateInternal( FRunnable*, const TCHAR*, uint32, EThreadPriority, uint64, EThreadCreateFlags) override { return true; }
public:
	FTBBThread()
	{
		static TAtomic<uint32> ThreadIndex(0);
		ThreadName = FString::Printf(TEXT("TBB %d"), ThreadIndex++);
		ThreadID = FPlatformTLS::GetCurrentThreadId();
		SetTls();
		UE::Trace::ThreadRegister(TEXT("TBB"), ThreadID, TPri_Normal);

		FPlatformProcess::SetThreadName(*ThreadName);
	}

	virtual ~FTBBThread() override
	{
		// Allow us to clear any TlsAutoCleanup
		FreeTls();
	}

	static FRunnableThread* GetRunnableThread()
	{
		return FRunnableThread::GetRunnableThread();
	}
};

class FTBBTaskObserver : private tbb::task_scheduler_observer
{
public:
	FTBBTaskObserver()
	{
	}

	void Initialize()
	{
		observe(true);
	}

	void Shutdown()
	{
		observe(false);
	}

private:
	void on_scheduler_entry(bool is_worker) override
	{
		if (is_worker)
		{
			if (FTBBThread::GetRunnableThread() == nullptr)
			{
				new FTBBThread();
			}
		}
	}

	void on_scheduler_exit(bool is_worker) override
	{
		if (is_worker)
		{
			if (FTBBThread::GetRunnableThread())
			{
				delete FTBBThread::GetRunnableThread();
			}
		}
	}
};

FTBBTaskObserver TBBTaskObserver;

void FProxyLODMeshReduction::StartupModule()
{
	TBBTaskObserver.Initialize();

	// Global registration of  the vdb types.
	openvdb::initialize();

	IModularFeatures::Get().RegisterModularFeature(IMeshReductionModule::GetModularFeatureName(), this);
}


void FProxyLODMeshReduction::ShutdownModule()
{
	// Global deregistration of vdb types
	openvdb::uninitialize();

	IModularFeatures::Get().UnregisterModularFeature(IMeshReductionModule::GetModularFeatureName(), this);

	TBBTaskObserver.Shutdown();
}


IMeshReduction*  FProxyLODMeshReduction::GetStaticMeshReductionInterface()
{
	return nullptr;
}

IMeshReduction*  FProxyLODMeshReduction::GetSkeletalMeshReductionInterface()
{
	return nullptr;
}

IMeshMerging* FProxyLODMeshReduction::GetDistributedMeshMergingInterface()  
{
	return nullptr;
}

IMeshMerging*  FProxyLODMeshReduction::GetMeshMergingInterface()
{

	return &MeshMergingTool;
}


FVoxelizeMeshMerging::FVoxelizeMeshMerging()
{}

FVoxelizeMeshMerging::~FVoxelizeMeshMerging()
{}


bool FVoxelizeMeshMerging::bSupportsParallelMaterialBake()
{
#if PROXYLOD_USE_TEST_CVARS	
	bool bDoParallel = (CVarProxyLODMaterialInParallel.GetValueOnAnyThread() == 1);
#else
	bool bDoParallel = true;
#endif 
	return bDoParallel;
}

void FVoxelizeMeshMerging::CaptureCVars()
{

#if PROXYLOD_USE_TEST_CVARS
	// Allow CVars to be used.
	{
		int32 RayOrder                 = CVarProxyLODTransfer.GetValueOnGameThread();
		int32 DilationSteps            = CVarProxyLODMaxDilationSteps.GetValueOnGameThread();
		bool bAddChartColorVerts       = (CVarProxyLODChartColorVerts.GetValueOnGameThread() == 1);
		bool bUseTrueTangentSpace      = (CVarProxyLODUseTangentSpace.GetValueOnGameThread() == 1);
		bool bVoxelizeAndRemeshOnly    = (CVarProxyLODRemeshOnly.GetValueOnGameThread() == 1);
		bool bSingleThreadedSimplify   = (CVarProxyLODSingleThreadSimplify.GetValueOnAnyThread() == 1);
		bool bWallCorreciton           = (CVarProxyLODCorrectCollapsedWalls.GetValueOnGameThread() == 1);

		// set values  - note, this class is a global (singleton) instance.
		this->RayHitOrder              = RayOrder;
		this->bChartColorVerts         = bAddChartColorVerts;
		this->bUseTangentSpace         = bUseTrueTangentSpace;
		this->bRemeshOnly              = bVoxelizeAndRemeshOnly;
		this->bMultiThreadSimplify     = !bSingleThreadedSimplify;
		this->bCorrectCollapsedWalls   = bWallCorreciton;
	}
#else
	RestoreDefaultParameters();
#endif 
}

void FVoxelizeMeshMerging::RestoreDefaultParameters()
{
	IsoSurfaceValue   = 0.5;
	RayHitOrder       = 1;
	MaxDilationSteps  = 7; 
	bChartColorVerts  = false;
	bUseTangentSpace  = true;
	bRemeshOnly       = false;
}

FVoxelizeMeshMerging::OpenVDBTransform::Ptr 
FVoxelizeMeshMerging::ComputeResolution(const FMeshProxySettings& InProxySettings, float ObjectSize)
{
	// Compute the required voxel size in world units.
	// if the requested voxel size is non-physical, use a default of 3
	double VoxelSize = 3.;
	int32 PixelCount = FMath::Max(InProxySettings.ScreenSize, (int32)50);
	PixelCount = FMath::Min(PixelCount, (int32)900);
	
	if (ObjectSize > 0.f ) 
	{
		// pixels per length scale
		const double LengthPerPixel = double(ObjectSize) / double(PixelCount);
		VoxelSize = ( LengthPerPixel * 1.95 / 3.); // magic scale.
	}
	
	// Maybe override the computed VoxelSize

	if (InProxySettings.bOverrideVoxelSize)
	{
		VoxelSize = InProxySettings.VoxelSize;
	}

	UE_LOGF(LogProxyLODMeshReduction, Log, "Spatial Sampling Distance Scale used %f, and the major axis for the object bbox was %f", VoxelSize, ObjectSize);
	
	return  OpenVDBTransform::createLinearTransform(VoxelSize);
}



typedef FAOSMesh           FSimplifierMeshType;


/**
* Replace the AOSMesh with a simplified version of itself.
*
* @param SrcGeometryPolyField   Container that holds the original geometry - primarily used to determine the number of polys the
*                               original geometry would bin in each parallel partition. 
* @param PixelCoverage          PixelCoverage and PercentToRetain are used in the heuristic that determines how much simplification is needed.          
* @param PercentToRetain
* @param InOutMesh              Simplfied mesh, replaces the input mesh.
* @param bSingleThreaded        Control for single threaded evaluation of the simplifier.
*/

static void SimplifyMesh( const FClosestPolyField& SrcGeometryPolyField, 
	                      const int32 PixelCoverage, 
	                      const float PercentToRetain, 
	                      FSimplifierMeshType& InOutMesh, 
	                      bool bSingleThreaded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SimplifyMesh)

	//SCOPE_LOG_TIME(TEXT("UE_ProxyLOD_Simplifier"), nullptr);

	// Compute some of the metrics that relate the desired resolution to simplifier parameters.
	const FMeshDescriptionArrayAdapter& SrcGeometryAdapter = SrcGeometryPolyField.MeshAdapter();
	const ProxyLOD::FBBox& SrcBBox = SrcGeometryAdapter.GetBBox();
	const float MaxSide = SrcBBox.extents().length();

	const float DistPerPixel = MaxSide / (float)PixelCoverage;

	// Determine the cost of removing a vert from a cube of the designated feature size.
	// NB: This is a magic number
	
	const float LengthScale = 170.f;
	float MaxFeatureCost = DistPerPixel * DistPerPixel * DistPerPixel * LengthScale;

	if (!bSingleThreaded)
	{
		ProxyLOD::ParallelSimplifyMesh(SrcGeometryPolyField, PercentToRetain, MaxFeatureCost, InOutMesh);
	}
	else
	{
		// The iso-surface mesh initially has more polys than the src geometry.
		// Setting the max number to retain to the poly count of the original geometry
		// will still result in a lot of simplification of the iso-surface mesh.

		const int32 MaxTriNumToRetain = (int32)SrcGeometryAdapter.polygonCount();
		const int32 MinTriNumToRetain = FMath::CeilToInt(MaxTriNumToRetain * PercentToRetain);
		ProxyLOD::FSimplifierTerminator Terminator(MinTriNumToRetain, MaxTriNumToRetain, MaxFeatureCost);
		ProxyLOD::SimplifyMesh(Terminator, InOutMesh);
	}

}

static float BBoxMajorAxisLength(const ProxyLOD::FBBox& BBox)
{
	return BBox.extents()[BBox.maxExtent()];
}

/**
* Compute the size of the UV texture atlas.  This will be square, and the length of a side
* will correspond to the longest side of requested textures.
*
* NB: A min size of 64x64 is enforced.
*/
static FIntPoint GetTexelGridSize(const FMaterialProxySettings& InMaterialSettings)
{
	FIntPoint MaxTextureSize = InMaterialSettings.GetMaxTextureSize();

	// Make the UVs square
	int32 MaxLength = FMath::Max(MaxTextureSize.X, MaxTextureSize.Y);
	MaxLength = FMath::Max(MaxLength, 64);
	return FIntPoint(MaxLength, MaxLength);
}

static ProxyLOD::ENormalComputationMethod GetNormalComputationMethod(const FMeshProxySettings& InProxySettings)
{
	ProxyLOD::ENormalComputationMethod Result = ProxyLOD::ENormalComputationMethod::AreaWeighted;
	const TEnumAsByte<EProxyNormalComputationMethod::Type>& Method = InProxySettings.NormalCalculationMethod;

	switch (InProxySettings.NormalCalculationMethod)
	{
		case EProxyNormalComputationMethod::AngleWeighted:
			Result = ProxyLOD::ENormalComputationMethod::AngleWeighted;
			break;

		case EProxyNormalComputationMethod::AreaWeighted:
			Result = ProxyLOD::ENormalComputationMethod::AreaWeighted;
			break;

		case EProxyNormalComputationMethod::EqualWeighted:
			Result = ProxyLOD::ENormalComputationMethod::EqualWeighted;
			break;
	
		default:
			checkSlow(0);
	}
	return Result;
}

void FVoxelizeMeshMerging::ProxyLOD(const FMeshMergeDataArray& InData, const FMeshProxySettings& InProxySettings, const FFlattenMaterialArray& InputMaterials, const FGuid InJobGUID)
{
	// Split the input meshes into two groups.  The main geometry and clipping geometry.
	// NB: only use pointers to avoid potentially copying a TArray of UV data.

	TArray<const FMeshMergeData*> InGeometry;
	TArray<const FMeshMergeData*> InClippingGeometry;

	for (int32 i = 0; i < InData.Num(); ++i)
	{
		const FMeshMergeData& MeshMergeData = InData[i];
		if (MeshMergeData.bIsClippingMesh)
		{
			InClippingGeometry.Add(&MeshMergeData);
		}
		else
		{
			InGeometry.Add(&MeshMergeData);
		}
	}

	// Create an adapter to make the data appear as a single mesh as required by the voxelization code.
	FMeshDescriptionArrayAdapter SrcGeometryAdapter(InGeometry);
	FMeshDescriptionArrayAdapter ClippingGeometryAdapter(InClippingGeometry);

	ProxyLOD(SrcGeometryAdapter, ClippingGeometryAdapter, InProxySettings, InputMaterials, InJobGUID);
}

void FVoxelizeMeshMerging::ProxyLOD(const FInstancedMeshMergeDataArray& InData, const FMeshProxySettings& InProxySettings, const FFlattenMaterialArray& InputMaterials, const FGuid InJobGUID)
{
	// Split the input meshes into two groups.  The main geometry and clipping geometry.
	// NB: only use pointers to avoid potentially copying a TArray of UV data.

	TArray<const FInstancedMeshMergeData*> InGeometry;
	TArray<const FInstancedMeshMergeData*> InClippingGeometry;

	for (int32 i = 0; i < InData.Num(); ++i)
	{
		const FInstancedMeshMergeData& MeshMergeData = InData[i];
		if (MeshMergeData.bIsClippingMesh)
		{
			InClippingGeometry.Add(&MeshMergeData);
		}
		else
		{
			InGeometry.Add(&MeshMergeData);
		}
	}

	// Create an adapter to make the data appear as a single mesh as required by the voxelization code.
	FMeshDescriptionArrayAdapter SrcGeometryAdapter(InGeometry);
	FMeshDescriptionArrayAdapter ClippingGeometryAdapter(InClippingGeometry);

	ProxyLOD(SrcGeometryAdapter, ClippingGeometryAdapter, InProxySettings, InputMaterials, InJobGUID);
}

void FVoxelizeMeshMerging::ProxyLOD(FMeshDescriptionArrayAdapter& InSrcGeometryAdapter, FMeshDescriptionArrayAdapter& InClippingGeometryAdapter, const FMeshProxySettings& InProxySettings, const FFlattenMaterialArray& InputMaterials, const FGuid InJobGUID)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVoxelizeMeshMerging::ProxyLOD)
	// Update any pipeline controlling parameters.

	CaptureCVars();

	if (InSrcGeometryAdapter.polygonCount() == 0)
	{
		UE_LOGF(LogProxyLODMeshReduction, Warning, "No static meshes input, no output mesh will be generated.");

		FailedDelegate.ExecuteIfBound(InJobGUID, TEXT("ProxyLOD no input geometry"));

		// Done with the material baking, free the delegate

		if (BakeMaterialsDelegate.IsBound())
		{
			BakeMaterialsDelegate.Unbind();
		}

		return;
	}

	FMaterialProxySettings MaterialSettings = InProxySettings.MaterialSettings;
	
	// Container for the raw mesh that will hold the simplified geometry
	// and the FlattenMaterial that will hold the materials for the output mesh.
	// NB: These will be the product of this function and 
	//     will be captured by the CompleteDelegate. 

	FMeshDescription OutRawMesh;
	FStaticMeshAttributes(OutRawMesh).Register();

	FFlattenMaterial OutMaterial;
	const FColor UnresolvedGeometryColor = InProxySettings.UnresolvedGeometryColor;

	bool bProxyGenerationSuccess = true;
	// Compute the simplified mesh and related materials.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputeSimplifiedMesh)

		{
			const auto& BBox = InSrcGeometryAdapter.GetBBox();

			if (!BBox.hasVolume())
			{
				UE_LOGF(LogProxyLODMeshReduction, Log, "Empty bounding box for all static meshes input, no output mesh will be generated.");

				FailedDelegate.ExecuteIfBound(InJobGUID, TEXT("ProxyLOD invalid input geometry"));

				// Done with the material baking, free the delegate

				if (BakeMaterialsDelegate.IsBound())
				{
					BakeMaterialsDelegate.Unbind();
				}

				return;
			}

			// Determine the voxel size the corresponds to the InProxySettings
			// The transform defines resolution of the voxel grid.

			OpenVDBTransform::Ptr XForm = ComputeResolution(InProxySettings, BBoxMajorAxisLength(BBox) );
			InSrcGeometryAdapter.SetTransform(XForm);
			InClippingGeometryAdapter.SetTransform(XForm);
		}

		const double VoxelSize = InSrcGeometryAdapter.GetTransform().voxelSize()[0];

		// Prepare kDOPTree asynchronously ahead of time so it's ready when we need it
		ProxyLOD::FkDOPTree  kDOPTree;
		ProxyLOD::FTaskGroup kDOPTaskGroup;

		kDOPTaskGroup.Run(
			[&kDOPTree, &InSrcGeometryAdapter]()
			{
				ProxyLOD::BuildkDOPTree(InSrcGeometryAdapter, kDOPTree);
			}
		);

		// --- Set pointers and containers shared by the various threaded stages ---
		// NB: These need to be declared outside of the thread task scope.

		// Bake the materials pointers and containers. 

		FFlattenMaterialArray LocalBakedMaterials;
		const FFlattenMaterialArray*  BakedMaterials = &InputMaterials;

		// Smart pointer use to manage memory

		FClosestPolyField::ConstPtr SrcGeometryPolyField;

		// Mesh types that will be shared by various stages.
		FSimplifierMeshType AOSMeshedVolume; 
		FVertexDataMesh VertexDataMesh;

		// Description of texture atlas.
		ProxyLOD::FTextureAtlasDesc TextureAtlasDesc;
		TextureAtlasDesc.Size = FIntPoint(64, 64);
		TextureAtlasDesc.Gutter = MaterialSettings.GutterSpace;
	    
		// --- Create New (High Poly) Geometry --
		// 1) Voxelize the source geometry & maybe gap fill.
		// 2) Extract high-poly surfaces
		// 3) Capture closest poly-field grid that allows quick identification of the poly closest to a voxel center.
		// 4) Transfer normals to the new geometry. 
		// The steps in the following scope are fully parallelized
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CreateHighPolyGeometry)
			{
				// Parameters for the voxelization and iso-surface extraction. 

				const double WSIsoValue = VoxelSize * this->IsoSurfaceValue; //  convert to world space units
				const double RemeshAdaptivity = 0.01f; // the re-meshing code does some internal adaptivity based on local normals.


				openvdb::FloatGrid::Ptr SDFVolume; 
				openvdb::Int32Grid::Ptr SrcPolyIndexGrid = openvdb::Int32Grid::create();

				// 1) Voxelize - this can potentially run out of memory when very large objecs (or very small voxel sizes) are used.
				
				const bool bSuccess = ProxyLOD::MeshToSDFVolume(InSrcGeometryAdapter, InSrcGeometryAdapter.GetTransform(), SDFVolume, SrcPolyIndexGrid.get());
				const bool bHasClipping = InClippingGeometryAdapter.polygonCount() != 0;

				if (bSuccess && bHasClipping)
				{
					// Voxelize the clipping geometry.

					openvdb::FloatGrid::Ptr SDFClipping;
					ProxyLOD::MeshToSDFVolume(InClippingGeometryAdapter, InClippingGeometryAdapter.GetTransform(), SDFClipping);

					// CSG difference that removes the clipping region from the SDFvolume, leaving a watertight SDF

					ProxyLOD::RemoveClipped(SDFVolume, SDFClipping);
				}

				if (bSuccess)
				{
					// Optionally manipulate the SDF to close potential gaps (e.g. windows and doors if the object is sufficiently far)
					
					double HoleRadius = 0.5 * InProxySettings.MergeDistance;
					openvdb::math::Coord VolumeBBoxSize = SDFVolume->evalActiveVoxelDim();
					
					// Clamp the hole radius.
					double BBoxMinorAxis = VolumeBBoxSize[VolumeBBoxSize.minIndex()] * VoxelSize;
					if (HoleRadius > .5 * BBoxMinorAxis )
					{
						HoleRadius = .5 * BBoxMinorAxis;
						UE_LOGF(LogProxyLODMeshReduction, Display, "Merge distance %f too large, clamped to %f.", InProxySettings.MergeDistance, float(2. * HoleRadius));
					}

					if (HoleRadius > 0.25 * VoxelSize && MaxDilationSteps > 0)
					{
						// performance tuning number.  if more dilations are required for this hole radius, a coarser grid is used.
						
						ProxyLOD::CloseGaps(SDFVolume, HoleRadius, MaxDilationSteps);
					}

					// 2) Extract the iso-surface into a mesh format directly consumable by the simplifier

					ProxyLOD::ExtractIsosurfaceWithNormals(SDFVolume, WSIsoValue, RemeshAdaptivity, AOSMeshedVolume);
				}
				else
				{
					UE_LOGF(LogProxyLODMeshReduction, Warning, "Allocation Error: The objects were too large for the selected Spatial Sampling Distance");

					// Make a simple cube for output.

					ProxyLOD::MakeCube(AOSMeshedVolume, 100);

					// Skip the UV and material steps

					this->bRemeshOnly = true;
					bProxyGenerationSuccess = false;
				}

				// 3) Create an object that allows for closest poly queries against the source mesh.

				SrcGeometryPolyField = FClosestPolyField::create(InSrcGeometryAdapter, SrcPolyIndexGrid);

				// Let the SDFVolume go out of scope, no longer needed.
			}

			// 4) Transfers the src geometry normals to the simplifier mesh.  

			ProxyLOD::TransferSrcNormals(*SrcGeometryPolyField, AOSMeshedVolume);
		}


		// Project the verts onto the source geometry.  This helps
		// insure that single-sided objects don't become too thick.

		ProxyLOD::ProjectVertexOntoSrcSurface(*SrcGeometryPolyField, AOSMeshedVolume);

		// Allow for the parallel execution of baking the source textures and generated the simplified geometry with UVs.

		bool bUVGenerationSuccess          = false;
		const bool bColorVertsByChart      = this->bChartColorVerts;
		const bool bDoCollapsedWallFix     = this->bCorrectCollapsedWalls;
		const bool bSingleThreadedSimplify = !(this->bMultiThreadSimplify);

		const float HardAngle       = InProxySettings.HardAngleThreshold;
		const bool bSplitHardAngles = (HardAngle > 0.f && HardAngle < 179.f) && InProxySettings.bUseHardAngleThreshold;
		
		if (!this->bRemeshOnly)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ConvertHighPolyIsoSurface)

			ProxyLOD::FTaskGroup PrepGeometryAndBakeMaterialsTaskGroup;
			
			// --- Convert High Poly Iso Surface To Simplified Geometry and Prepare for Materials ---
			// 1) Simplify Geometry
			// 2) Compute Normals for Simplified Geometry
			// 3) Generate UVs for Simplified Geometry

			PrepGeometryAndBakeMaterialsTaskGroup.Run([&AOSMeshedVolume,
				&SrcGeometryPolyField, &VertexDataMesh,
				&TextureAtlasDesc, &MaterialSettings, &bUVGenerationSuccess, &InProxySettings,
				VoxelSize, bColorVertsByChart, bSingleThreadedSimplify, bDoCollapsedWallFix, bSplitHardAngles, HardAngle]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(PrepGeometryTask)
				// 1) Simplified mesh and convert it to the correct format for UV generation

				{

					// Compute the number target number of polys.

					const int32 PixelCoverage = InProxySettings.ScreenSize;

					// By default, we don't want the simplifier to toss much more than 98% of the triangles.

					float PercentToRetain = 0.002f; 

					// Replaces the AOS mesh with a simplified version

					FSimplifierMeshType& SimplifierMesh = AOSMeshedVolume;
					SimplifyMesh(*SrcGeometryPolyField, PixelCoverage, PercentToRetain, SimplifierMesh, bSingleThreadedSimplify);

					// Project the verts onto the source geometry.  This helps
					// insure that single-sided objects don't become too thick.

					//ProjectVerticesOntoSrc(*SrcRefernce, AOSMeshedVolume);

					// Convert to format used by the UV generation code

					ProxyLOD::ConvertMesh(SimplifierMesh, VertexDataMesh);

					// --- Attempt to fix-up the geometry if needed ---

					if (bDoCollapsedWallFix)
					{
						// Use heuristics to try to improve the finial mesh by comparing with the source meshes.
						// Todo:  try moving this to before the Correspondence creation.
						// 	ProjectVerticesOntoSrc<true /*snap to vertex*/>(*SrcGeometryPolyField, OutRawMesh);

						// Very thin walls may have generated inter-penetration. Attempt to fix.

						int32 testCount = ProxyLOD::CorrectCollapsedWalls(VertexDataMesh, VoxelSize);
					}
				}

				// 2) Add angle weighted vertex normals

				ProxyLOD::ComputeVertexNormals(VertexDataMesh, ProxyLOD::ENormalComputationMethod::AngleWeighted);
				// These normals will be used in making the correspondence with the original geometry
				ProxyLOD::CacheNormals(VertexDataMesh);   

				if (bSplitHardAngles)
				{
					// Split the verts on the hard angles

					ProxyLOD::SplitHardAngles(HardAngle, VertexDataMesh);
				}

				// Compute the vertex normals using the possibly updated connectivity. 
				const ProxyLOD::ENormalComputationMethod NormalMethod = GetNormalComputationMethod(InProxySettings);
				ProxyLOD::ComputeVertexNormals(VertexDataMesh, NormalMethod);


				// 3) Resolve texture size if required
				float WorldSpaceRadius = ProxyLOD::GetBounds(VertexDataMesh).SphereRadius;
				double WorldSpaceArea = ProxyLOD::GetWorldSpaceArea(VertexDataMesh);
				MaterialSettings.ResolveTextureSize(WorldSpaceRadius, WorldSpaceArea);
				TextureAtlasDesc.Size = GetTexelGridSize(MaterialSettings);

				// 4) Generate UVs for Simplified Geometry
				// UV Atlas create, this can fail.
				// NB: Vertices are split on UV seams. 

				bUVGenerationSuccess = ProxyLOD::GenerateUVs(VertexDataMesh, TextureAtlasDesc, bColorVertsByChart);
			});

			// --- Bake the materials ---
			// This only needs to be done before the material transfer step, but it can be the slowest
			// step in the process, so we overlap simplification by doing it in this task group.
			//
			// This guarantees that the BakeMaterialsDelegate task will run on the main thread ( i.e. the game thread)
			// NB: The BakeMaterialsDelegate is required by other code to run on the game thread
			// The wait acts like a join, e.g. both tasks will complete.

			PrepGeometryAndBakeMaterialsTaskGroup.RunAndWait(
				[&]()->void 
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(BakeMaterialTask)
					if (this->bSupportsParallelMaterialBake() && BakeMaterialsDelegate.IsBound())
					{
						IMeshMerging::BakeMaterialsDelegate.Execute(LocalBakedMaterials);
						BakedMaterials = &LocalBakedMaterials;
					}
				}
			);
			InSrcGeometryAdapter.UpdateMaterialsID();
		}
		else
		{
			// Convert to the expected mesh type.

			ProxyLOD::ConvertMesh(AOSMeshedVolume, VertexDataMesh);
		}

		// Update the locations of the vertexes.  This has no affect on the tangent space
		// ProxyLOD::ProjectVertexWithSnapToNearest(*SrcGeometryPolyField, VertexDataMesh);

		// Using the new UVs fill OutMaterial texture atlas. 

		OutMaterial = FMaterialUtilities::CreateFlattenMaterialWithSettings(MaterialSettings);

		if (bUVGenerationSuccess && SrcGeometryPolyField)
		{
			ProxyLOD::FRasterGrid::Ptr DstUVGrid;
			ProxyLOD::FRasterGrid::Ptr DstSuperSampledUVGrid;

			ProxyLOD::FTaskGroup MapTextureAtlasAndAddTangentSpaceTaskGroup;


			// -- Map the texture atlas texels to triangles on the Simplified Geometry --
			// 1) Map at final resolution
			// 2) Map at supper sample resolution
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MapTextureAtlas)

				// 1) Rasterize the triangles into a grid of the same resolution as the texture atlas.

				MapTextureAtlasAndAddTangentSpaceTaskGroup.Run(
					[&DstUVGrid, &VertexDataMesh, &TextureAtlasDesc]()->void
				{
					DstUVGrid = ProxyLOD::RasterizeTriangles(VertexDataMesh, TextureAtlasDesc, 2/*padding*/);
				}
				);

				// 2) Generate a UV Grid used to map the mesh UVs to texels.

				MapTextureAtlasAndAddTangentSpaceTaskGroup.Run(
					[&VertexDataMesh, &TextureAtlasDesc, &DstSuperSampledUVGrid]()->void
				{
					const int32 SuperSampleNum = 16;

					DstSuperSampledUVGrid = ProxyLOD::RasterizeTriangles(VertexDataMesh, TextureAtlasDesc, 1/*padding*/, SuperSampleNum/* super samples*/);
				}
				);
			}


			// --- Compute the tangent space on the simplified mesh ---
			// Convert to FMeshDescription because it has the per-index attribute structure needed for the tangent space.
			// Generate the tangent space, but retain our current normals.

			MapTextureAtlasAndAddTangentSpaceTaskGroup.RunAndWait(
				[&VertexDataMesh, &OutRawMesh]()->void
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(ComputeTangentSpace)

					const bool bDoMikkT = false;
					const bool bRecomputeNormals = false; // if true, the UV seams are often more apparent since verts may have been split to make uvs. 
					if (bDoMikkT)
					{
						ProxyLOD::ConvertMesh(VertexDataMesh, OutRawMesh);

						// Compute Mikk-T version of tangent space.  Requires the smoothing groups, and 
						// is slower than the per-vertex  tangent space computation.

						ProxyLOD::ComputeTangentSpace(OutRawMesh, bRecomputeNormals);
					}
					else
					{
						// Per-vertex tangent space computation

						ProxyLOD::ComputeTangentSpace(VertexDataMesh, bRecomputeNormals);

						ProxyLOD::ConvertMesh(VertexDataMesh, OutRawMesh);
					}

				}
			);

			// --- Populate the output materials ---
			// --- By mapping the texels to the source geometry.
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(PopulateOutputMaterials)

				// Inherit the material settings from the first baked down source material.

				if (BakedMaterials->Num())
				{
					const auto& FirstMaterial = (*BakedMaterials)[0];
					OutMaterial.BlendMode              = FirstMaterial.BlendMode;

					for (const FFlattenMaterial& FlatMaterial : (*BakedMaterials))
					{
						OutMaterial.bTwoSided |= FlatMaterial.bTwoSided;
						OutMaterial.bDitheredLODTransition |= FlatMaterial.bDitheredLODTransition;
						OutMaterial.EmissiveScale = FMath::Max(OutMaterial.EmissiveScale, FlatMaterial.EmissiveScale);
					}
				}

				// --- Map a correspondence between texels in the texture atlas and locations on the source geometry --

				// The termination distance when we fire rays.  These rays should only be traveling  short 
				// distance between polygons in the simplified geometry and polygons in the source geometry.
				const bool bOverrideRayDist = InProxySettings.bOverrideTransferDistance;
				double RequestedMaxRay = (bOverrideRayDist) ? InProxySettings.MaxRayCastDist : 6. * VoxelSize;
				const double BBoxMajorAxisLength = InSrcGeometryAdapter.GetBBox().extents()[InSrcGeometryAdapter.GetBBox().maxExtent()];
				const double MaxRayLength = FMath::Max(VoxelSize, FMath::Min(RequestedMaxRay, BBoxMajorAxisLength));

				UE_LOGF(LogProxyLODMeshReduction, Log, "Material Distance %f used to discover textures.", float(MaxRayLength));

				// Fire rays from the simplified mesh to the original collection of meshes 
				// to determine a correspondence between the SrcMesh and the Simplified mesh.

				// Now that kDOPTree is needed, make sure it's ready
				kDOPTaskGroup.Wait();

				ProxyLOD::FSrcDataGrid::Ptr SuperSampledCorrespondenceGrid =
					ProxyLOD::CreateCorrespondence(InSrcGeometryAdapter, kDOPTree, VertexDataMesh, *DstSuperSampledUVGrid, this->RayHitOrder, MaxRayLength);

				// just for testing, this should force it to save in world space
				const bool bUseWorldSpaceNormals = (!this->bUseTangentSpace);
				if (bUseWorldSpaceNormals)
				{
					ProxyLOD::ComputeBogusNormalTangentAndBiTangent(VertexDataMesh);
					ProxyLOD::ConvertMesh(VertexDataMesh, OutRawMesh);
				}

				// --- Use the correspondence between texels in the texture atlas and simplified geometry ---
				// --- and the correspondence between the same texels and the source geometry            ---
				// --- to generate flattened materials for the simplified geometry                       ---
				// Compute the baked-down maps

				ProxyLOD::MapFlattenMaterials(OutRawMesh, InSrcGeometryAdapter, *SuperSampledCorrespondenceGrid, *DstSuperSampledUVGrid, *DstUVGrid, *BakedMaterials, UnresolvedGeometryColor, OutMaterial);

				// Transfer the vertex colors unless we want to display the Charts as vertex colors for debugging

				bool bTransferVertexColors = (!this->bChartColorVerts);
				if (bTransferVertexColors)
				{
					ProxyLOD::TransferVertexColors(*SrcGeometryPolyField, OutRawMesh);
				}
			}
		}
		else  // UV-generation failed! 
		{
			bProxyGenerationSuccess = false;

			UE_LOGF(LogProxyLODMeshReduction, Warning, "UV Generation failed.");

			// --- Add default UVs and tangents to the mesh and add a Red Diffuse material ---

			// Convert the mesh with bogus UVs and tangents.

			ProxyLOD::ConvertMesh(VertexDataMesh, OutRawMesh);

			// Generate default out-textures.  Color the failed mesh red.
			
			const FIntPoint DstSize      = OutMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse);
			TArray<FColor>& TargetBuffer = OutMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse);
			ResizeArray(TargetBuffer, DstSize.X * DstSize.Y);
			for (int32 i = 0; i < DstSize.X * DstSize.Y; ++i) TargetBuffer[i] = FColor::Red;
		}
	}

	// testing 
	bProxyGenerationSuccess = bProxyGenerationSuccess || (this->bRemeshOnly);

	if (bProxyGenerationSuccess)
	{

		// Revert the smoothing group to a default.
		TEdgeAttributesRef<bool> EdgeHardnesses = OutRawMesh.EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard);

		for (const FEdgeID& EdgeID : OutRawMesh.Edges().GetElementIDs())
		{
			EdgeHardnesses[EdgeID] = false;
		}


		// NB: FProxyGenerationProcessor::ProxyGenerationComplete
		CompleteDelegate.ExecuteIfBound(OutRawMesh, OutMaterial, InJobGUID);
	}
	else
	{
		FailedDelegate.ExecuteIfBound(InJobGUID, TEXT("ProxyLOD UV Generation failed"));
	}


	// Done with the material baking, free the delegate

	if (BakeMaterialsDelegate.IsBound())
	{
		BakeMaterialsDelegate.Unbind();
	}

}

#undef LOCTEXT_NAMESPACE