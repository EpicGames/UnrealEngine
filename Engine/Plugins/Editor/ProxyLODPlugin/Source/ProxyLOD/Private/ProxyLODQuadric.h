// Copyright (C) 2009 Nine Realms, Inc
//==================================================================================================
// These methods and classes are slightly modified copies from Developer/MeshSimplifier/Private/Quadric.{h,cpp}
// and hence the copyright.

#pragma once

#include "CoreMinimal.h"
#include <cmath>

#define SINGULAR_QUADRIC_DET 1e-8

DEFINE_LOG_CATEGORY_STATIC(LogProxyQuadric, Log, All);


#define PROXYLOD_WEIGHT_BY_AREA		1
#define PROXYLOD_VOLUME_CONSTRAINT	1

namespace ProxyLOD
{


	static FORCEINLINE void CalcGradient(double* __restrict GradMatrix, float a0, float a1, float a2, double OutGrad[4])
	{
		OutGrad[0] = -GradMatrix[0] * a0 + GradMatrix[4] * a1 + GradMatrix[ 8] * a2;
		OutGrad[1] = +GradMatrix[1] * a0 - GradMatrix[5] * a1 - GradMatrix[ 9] * a2;
		OutGrad[2] = -GradMatrix[2] * a0 + GradMatrix[6] * a1 + GradMatrix[10] * a2;
		OutGrad[3] = +GradMatrix[3] * a0 - GradMatrix[7] * a1 - GradMatrix[11] * a2;
	}

	// returns length and normalizes input vector

	static FORCEINLINE double NormalizeSelf(double(&V)[3])
	{
		double Length2 = V[0] * V[0] + V[1] * V[1] + V[2] * V[2];
		double Length = std::sqrt(Length2);
		{
			double InvL = 1. / Length;
			V[0] *= InvL;
			V[1] *= InvL;
			V[2] *= InvL;
		}
		return Length;
	}

	static FORCEINLINE double NormalizeSelf(double& Vx, double& Vy, double& Vz)
	{
		double Length2 = Vx * Vx + Vy * Vy + Vz * Vz;
		double Length = std::sqrt(Length2);
		{
			double InvL = 1. / Length;
			Vx *= InvL;
			Vy *= InvL;
			Vz *= InvL;
		}
		return Length;
	}

	static bool CalcGradientMatrix(const double(&p0)[3], const double(&p1)[3], const double(&p2)[3], const double(&n)[3], double* __restrict OutGradMatrix)
	{
		// solve for gd
		// [ p0, 1 ][ g0 ] = [ a0 ]
		// [ p1, 1 ][ g1 ] = [ a1 ]
		// [ p2, 1 ][ g2 ] = [ a2 ]
		// [ n,  0 ][ d  ] = [ 0  ]

		double det, invDet;

		// 2x2 sub-determinants required to calculate 4x4 determinant
		double det2_01_01 = p0[0] * p1[1] - p0[1] * p1[0];
		double det2_01_02 = p0[0] * p1[2] - p0[2] * p1[0];
		double det2_01_03 = p0[0] - p1[0];
		double det2_01_12 = p0[1] * p1[2] - p0[2] * p1[1];
		double det2_01_13 = p0[1] - p1[1];
		double det2_01_23 = p0[2] - p1[2];

		// 3x3 sub-determinants required to calculate 4x4 determinant
		double det3_201_013 = p2[0] * det2_01_13 - p2[1] * det2_01_03 + det2_01_01;
		double det3_201_023 = p2[0] * det2_01_23 - p2[2] * det2_01_03 + det2_01_02;
		double det3_201_123 = p2[1] * det2_01_23 - p2[2] * det2_01_13 + det2_01_12;

		det = -det3_201_123 * n[0] + det3_201_023 * n[1] - det3_201_013 * n[2];

		if (FMath::Abs(det) < SINGULAR_QUADRIC_DET)
		{
			return false;
		}

		invDet = 1.0 / det;

		// remaining 2x2 sub-determinants
		double det2_03_01 = p0[0] * n[1] - p0[1] * n[0];
		double det2_03_02 = p0[0] * n[2] - p0[2] * n[0];
		double det2_03_12 = p0[1] * n[2] - p0[2] * n[1];
		double det2_03_03 = -n[0];
		double det2_03_13 = -n[1];
		double det2_03_23 = -n[2];

		double det2_13_01 = p1[0] * n[1] - p1[1] * n[0];
		double det2_13_02 = p1[0] * n[2] - p1[2] * n[0];
		double det2_13_12 = p1[1] * n[2] - p1[2] * n[1];
		double det2_13_03 = -n[0];
		double det2_13_13 = -n[1];
		double det2_13_23 = -n[2];

		// remaining 3x3 sub-determinants
		double det3_203_012 = p2[0] * det2_03_12 - p2[1] * det2_03_02 + p2[2] * det2_03_01;
		double det3_203_013 = p2[0] * det2_03_13 - p2[1] * det2_03_03 + det2_03_01;
		double det3_203_023 = p2[0] * det2_03_23 - p2[2] * det2_03_03 + det2_03_02;
		double det3_203_123 = p2[1] * det2_03_23 - p2[2] * det2_03_13 + det2_03_12;

		double det3_213_012 = p2[0] * det2_13_12 - p2[1] * det2_13_02 + p2[2] * det2_13_01;
		double det3_213_013 = p2[0] * det2_13_13 - p2[1] * det2_13_03 + det2_13_01;
		double det3_213_023 = p2[0] * det2_13_23 - p2[2] * det2_13_03 + det2_13_02;
		double det3_213_123 = p2[1] * det2_13_23 - p2[2] * det2_13_13 + det2_13_12;

		double det3_301_012 = n[0] * det2_01_12 - n[1] * det2_01_02 + n[2] * det2_01_01;
		double det3_301_013 = n[0] * det2_01_13 - n[1] * det2_01_03;
		double det3_301_023 = n[0] * det2_01_23 - n[2] * det2_01_03;
		double det3_301_123 = n[1] * det2_01_23 - n[2] * det2_01_13;

		OutGradMatrix[0] = det3_213_123 * invDet;
		OutGradMatrix[1] = det3_213_023 * invDet;
		OutGradMatrix[2] = det3_213_013 * invDet;
		OutGradMatrix[3] = det3_213_012 * invDet;

		OutGradMatrix[4] = det3_203_123 * invDet;
		OutGradMatrix[5] = det3_203_023 * invDet;
		OutGradMatrix[6] = det3_203_013 * invDet;
		OutGradMatrix[7] = det3_203_012 * invDet;

		OutGradMatrix[8] = det3_301_123 * invDet;
		OutGradMatrix[9] = det3_301_023 * invDet;
		OutGradMatrix[10] = det3_301_013 * invDet;
		OutGradMatrix[11] = det3_301_012 * invDet;

		return true;
	}


	// Error quadric for position only
	class FQuadric
	{
	public:
		FQuadric() {}

		// Quadric for triangle
		FQuadric(const FVector& p0, const FVector& p1, const FVector& p2);
		// Quadric for boundary edge
		FQuadric(const FVector& p0, const FVector& p1, const FVector& faceNormal, const float edgeWeight);

		void		Zero();

		FQuadric&	operator+=(const FQuadric& q);

		// Evaluate error at point
		float		Evaluate(const FVector& p) const;

		double		nxx;
		double		nyy;
		double		nzz;

		double		nxy;
		double		nxz;
		double		nyz;

		double		dnx;
		double		dny;
		double		dnz;

		double		d2;

		double		a;
	};

	inline FQuadric::FQuadric(const FVector& fp0, const FVector& fp1, const FVector& fp2)
	{
		// Convert to double 

		const double p0[3] = { fp0[0], fp0[1], fp0[2] };
		const double p1[3] = { fp1[0], fp1[1], fp1[2] };
		const double p2[3] = { fp2[0], fp2[1], fp2[2] };

		// Compute the wedge product, giving the normal direction scaled by 
		// twice the triangle area.

		double nX, nY, nZ;
		{
			double tmpA[3] = { p2[0] - p0[0], p2[1] - p0[1], p2[2] - p0[2] };
			double tmpB[3] = { p1[0] - p0[0], p1[1] - p0[1], p1[2] - p0[2] };

			nX = tmpA[1] * tmpB[2] - tmpA[2] * tmpB[1];
			nY = tmpA[2] * tmpB[0] - tmpA[0] * tmpB[2];
			nZ = tmpA[0] * tmpB[1] - tmpA[1] * tmpB[0];
		}

		// Rescale the normal direction vector to unit length.

		double Length = NormalizeSelf(nX, nY, nZ);
		if (Length < double(SMALL_NUMBER))
		{
			Zero();
			return;
		}

		checkSlow(FMath::IsFinite(nX));
		checkSlow(FMath::IsFinite(nY));
		checkSlow(FMath::IsFinite(nZ));

		const double area = 0.5 * Length;
		const double dist = -(nX * p0[0] + nY * p0[1] + nZ * p0[2]); // n.dot.p0

		nxx = nX * nX;
		nyy = nY * nY;
		nzz = nZ * nZ;

		nxy = nX * nY;
		nxz = nX * nZ;
		nyz = nY * nZ;

		dnx = dist * nX;
		dny = dist * nY;
		dnz = dist * nZ;

		d2 = dist * dist;

#if PROXYLOD_WEIGHT_BY_AREA
		nxx *= area;
		nyy *= area;
		nzz *= area;

		nxy *= area;
		nxz *= area;
		nyz *= area;

		dnx *= area;
		dny *= area;
		dnz *= area;

		d2 *= area;

		a = area;
#else
		a = 1.0;
#endif
	}

	inline FQuadric::FQuadric(const FVector& fp0, const FVector& fp1, const FVector& faceNormal, const float edgeWeight)
	{
		if (!faceNormal.IsNormalized())
		{
			Zero();
			return;
		}

		// Convert to double 

		const double p0[3] = { fp0[0], fp0[1], fp0[2] };
		const double p1[3] = { fp1[0], fp1[1], fp1[2] };

		// Compute the wedge product, giving the a direction
		// normal to the edge and face normal (a bi-tangent) 

		double nX, nY, nZ; //  n = (p1 - p0) ^ (faceNormal);
		double edgeSize;
		{
			// edge = fp1 - fp0
			double tmpA[3] = { p1[0] - p0[0], p1[1] - p0[1], p1[2] - p0[2] };
			double tmpB[3] = { faceNormal.X,  faceNormal.Y,  faceNormal.Z };

			nX = tmpA[1] * tmpB[2] - tmpA[2] * tmpB[1];
			nY = tmpA[2] * tmpB[0] - tmpA[0] * tmpB[2];
			nZ = tmpA[0] * tmpB[1] - tmpA[1] * tmpB[0];

			edgeSize = std::sqrt(tmpA[0] * tmpA[0] + tmpA[1] * tmpA[1] + tmpA[2] * tmpA[2]);
		}

		// Rescale the normal direction vector to unit length.

		const double Length = NormalizeSelf(nX, nY, nZ);
		if (Length < double(SMALL_NUMBER))
		{
			Zero();
			return;
		}

		checkSlow(FMath::IsFinite(nX));
		checkSlow(FMath::IsFinite(nY));
		checkSlow(FMath::IsFinite(nZ));

		double dist = -(nX * p0[0] + nY * p0[1] + nZ * p0[2]); // n.dot.p0
		double weight = edgeWeight * edgeSize;

		nxx = weight * nX * nX;
		nyy = weight * nY * nY;
		nzz = weight * nZ * nZ;

		nxy = weight * nX * nY;
		nxz = weight * nX * nZ;
		nyz = weight * nY * nZ;

		dnx = weight * dist * nX;
		dny = weight * dist * nY;
		dnz = weight * dist * nZ;

		d2 = weight * dist * dist;

		a = 0.0;
	}

	inline void FQuadric::Zero()
	{
		nxx = 0.0;
		nyy = 0.0;
		nzz = 0.0;

		nxy = 0.0;
		nxz = 0.0;
		nyz = 0.0;

		dnx = 0.0;
		dny = 0.0;
		dnz = 0.0;

		d2 = 0.0;

		a = 0.0;
	}

	inline FQuadric& FQuadric::operator+=(const FQuadric& q)
	{
		nxx += q.nxx;
		nyy += q.nyy;
		nzz += q.nzz;

		nxy += q.nxy;
		nxz += q.nxz;
		nyz += q.nyz;

		dnx += q.dnx;
		dny += q.dny;
		dnz += q.dnz;

		d2 += q.d2;

		a += q.a;

		return *this;
	}

	inline float FQuadric::Evaluate(const FVector& Point) const
	{
		// Q(v) = vt*A*v + 2*bt*v + c

		// v = [ p ]
		//     [ s ]

		// A = [ C  B  ]
		//     [ Bt aI ]

		// C = n*nt
		// B = -g[ 0 .. m ]

		// b = [  dn         ]
		//     [ -d[ 0 .. m] ]

		// c = d2

		double px = Point.X;
		double py = Point.Y;
		double pz = Point.Z;

		// A*v = [ C*p  + B*s ]
		//       [ Bt*p + a*s ]

		// C*p
		double x = px * nxx + py * nxy + pz * nxz;
		double y = px * nxy + py * nyy + pz * nyz;
		double z = px * nxz + py * nyz + pz * nzz;

		// vt*A*v = pt * ( C*p + B*s ) + st * ( Bt*p + a*s )

		// pt * (C*p + B*s)
		double vAv = px * x + py * y + pz * z;

		// bt*v
		double btv = px * dnx + py * dny + pz * dnz;

		// Q(v) = vt*A*v + 2*bt*v + c
		double Q = vAv + 2.0 * btv + d2;

		//check( Q > -1.0 );
		if (!FMath::IsFinite(Q))
		{
			UE_LOGF(LogProxyQuadric, Warning, "Quadric point evaluate detected possible degenerate. Returning 0.");
			Q = 0;
		}

		return Q;
	}


	// Error quadric including attributes
	template< uint32 NumAttributes >
	class TQuadricAttr
	{
	public:
		TQuadricAttr() {}
		TQuadricAttr(
			const FVector& p0, const FVector& p1, const FVector& p2,
			const float* a0, const float* a1, const float* a2,
			const float* AttributeWeights
		);

		void		Zero();

		TQuadricAttr< NumAttributes >& operator+=(const FQuadric& q);
		TQuadricAttr< NumAttributes >& operator+=(const TQuadricAttr< NumAttributes >& q);

		// Evaluate error at point with attributes and weights
		float		Evaluate(const FVector& Point, const float* Attributes, const float* AttributeWeights) const;

		// Calculate attributes for point
		void		CalcAttributes(const FVector& Point, float* Attributes, const float* AttributeWeights) const;

		double		nxx;
		double		nyy;
		double		nzz;

		double		nxy;
		double		nxz;
		double		nyz;

		double		dnx;
		double		dny;
		double		dnz;

		double		d2;

		double		g[NumAttributes][3];
		double		d[NumAttributes];

		double		a;

#if PROXYLOD_VOLUME_CONSTRAINT
		double		nvx;
		double		nvy;
		double		nvz;
		double		dv;
#endif
	};

	template< uint32 NumAttributes >
	inline TQuadricAttr< NumAttributes >::TQuadricAttr(
		const FVector& fp0, const FVector& fp1, const FVector& fp2,
		const float* attr0, const float* attr1, const float* attr2,
		const float* AttributeWeights)
	{
		// Convert to double 

		const double p0[3] = { fp0[0], fp0[1], fp0[2] };
		const double p1[3] = { fp1[0], fp1[1], fp1[2] };
		const double p2[3] = { fp2[0], fp2[1], fp2[2] };

		// Compute the wedge product, giving the normal direction scaled by 
		// twice the triangle area.

		double n[3];
		{
			double tmpA[3] = { p2[0] - p0[0], p2[1] - p0[1], p2[2] - p0[2] };
			double tmpB[3] = { p1[0] - p0[0], p1[1] - p0[1], p1[2] - p0[2] };

			n[0] = tmpA[1] * tmpB[2] - tmpA[2] * tmpB[1];
			n[1] = tmpA[2] * tmpB[0] - tmpA[0] * tmpB[2];
			n[2] = tmpA[0] * tmpB[1] - tmpA[1] * tmpB[0];
		}

		// Rescale the normal direction vector to unit length.

		double Length = NormalizeSelf(n);
		if (Length < double(SMALL_NUMBER))
		{
			Zero();
			return;
		}

		checkSlow(FMath::IsFinite(n[0]));
		checkSlow(FMath::IsFinite(n[1]));
		checkSlow(FMath::IsFinite(n[2]));

		const double area = 0.5 * Length;
		const double dist = -(n[0] * p0[0] + n[1] * p0[1] + n[2] * p0[2]); // n.dot.p0

		const double nX = n[0];
		const double nY = n[1];
		const double nZ = n[2];
		
		nxx = nX * nX;
		nyy = nY * nY;
		nzz = nZ * nZ;

		nxy = nX * nY;
		nxz = nX * nZ;
		nyz = nY * nZ;

		dnx = dist * nX;
		dny = dist * nY;
		dnz = dist * nZ;

		d2 = dist * dist;

#if PROXYLOD_VOLUME_CONSTRAINT
		nvx = nX * (area / 3.0);
		nvy = nY * (area / 3.0);
		nvz = nZ * (area / 3.0);
		dv = dist * (area / 3.0);
#endif

		double GradMatrix[12];
		bool bInvertable = CalcGradientMatrix(p0, p1, p2, n, GradMatrix);

		for (uint32 i = 0; i < NumAttributes; i++)
		{
			if (AttributeWeights[i] == 0.0f)
			{
				g[i][0] = 0.0;
				g[i][1] = 0.0;
				g[i][2] = 0.0;
				d[i] = 0.0;
				continue;
			}

			float a0 = AttributeWeights[i] * attr0[i];
			float a1 = AttributeWeights[i] * attr1[i];
			float a2 = AttributeWeights[i] * attr2[i];

			a0 = FMath::IsFinite(a0) ? a0 : 0.0f;
			a1 = FMath::IsFinite(a1) ? a1 : 0.0f;
			a2 = FMath::IsFinite(a2) ? a2 : 0.0f;

			double grad[4] = { 0., 0., 0., (a0 + a1 + a2) / 3.0 }; // default values for non-invert case
			if (bInvertable)
			{
				CalcGradient(GradMatrix, a0, a1, a2, grad);

			}
			else
			{
				a0 = FMath::IsFinite(a0) ? a0 : 0.0f;
				a1 = FMath::IsFinite(a1) ? a1 : 0.0f;
				a2 = FMath::IsFinite(a2) ? a2 : 0.0f;

				CalcGradient(GradMatrix, a0, a1, a2, grad);
			}

			checkSlow(!FMath::IsNaN(grad[0]));
			checkSlow(!FMath::IsNaN(grad[1]));
			checkSlow(!FMath::IsNaN(grad[2]));
			checkSlow(!FMath::IsNaN(grad[3]));

			//double t0 = grad[0] * p0.x + grad[1] * p0.y + grad[2] * p0.z + grad[3];
			//double t1 = grad[0] * p1.x + grad[1] * p1.y + grad[2] * p1.z + grad[3];
			//double t2 = grad[0] * p2.x + grad[1] * p2.y + grad[2] * p2.z + grad[3];

			//assert( abs( a0 - t0 ) < 0.01 );
			//assert( abs( a1 - t1 ) < 0.01 );
			//assert( abs( a2 - t2 ) < 0.01 );

			g[i][0] = grad[0];
			g[i][1] = grad[1];
			g[i][2] = grad[2];
			d[i] = grad[3];

			nxx += g[i][0] * g[i][0];
			nyy += g[i][1] * g[i][1];
			nzz += g[i][2] * g[i][2];

			nxy += g[i][0] * g[i][1];
			nxz += g[i][0] * g[i][2];
			nyz += g[i][1] * g[i][2];

			dnx += d[i] * g[i][0];
			dny += d[i] * g[i][1];
			dnz += d[i] * g[i][2];

			d2 += d[i] * d[i];
		}

#if PROXYLOD_WEIGHT_BY_AREA
		nxx *= area;
		nyy *= area;
		nzz *= area;

		nxy *= area;
		nxz *= area;
		nyz *= area;

		dnx *= area;
		dny *= area;
		dnz *= area;

		d2 *= area;

		for (int i = 0; i < NumAttributes; i++)
		{
			g[i][0] *= area;
			g[i][1] *= area;
			g[i][2] *= area;
			d[i] *= area;
		}

		a = area;
#else
		a = 1.0;
#endif
	}

	template< uint32 NumAttributes >
	inline void TQuadricAttr< NumAttributes >::Zero()
	{
		nxx = 0.0;
		nyy = 0.0;
		nzz = 0.0;

		nxy = 0.0;
		nxz = 0.0;
		nyz = 0.0;

		dnx = 0.0;
		dny = 0.0;
		dnz = 0.0;

		d2 = 0.0;

		for (uint32 i = 0; i < NumAttributes; i++)
		{
			g[i][0] = 0.0;
			g[i][1] = 0.0;
			g[i][2] = 0.0;
			d[i] = 0.0;
		}

		a = 0.0;

#if PROXYLOD_VOLUME_CONSTRAINT
		nvx = 0.0;
		nvy = 0.0;
		nvz = 0.0;
		dv = 0.0;
#endif
	}

	template< uint32 NumAttributes >
	inline TQuadricAttr< NumAttributes >& TQuadricAttr< NumAttributes >::operator+=(const FQuadric& q)
	{
		nxx += q.nxx;
		nyy += q.nyy;
		nzz += q.nzz;

		nxy += q.nxy;
		nxz += q.nxz;
		nyz += q.nyz;

		dnx += q.dnx;
		dny += q.dny;
		dnz += q.dnz;

		d2 += q.d2;

		a += q.a;

		return *this;
	}

	template< uint32 NumAttributes >
	inline TQuadricAttr< NumAttributes >& TQuadricAttr< NumAttributes >::operator+=(const TQuadricAttr< NumAttributes >& q)
	{
		nxx += q.nxx;
		nyy += q.nyy;
		nzz += q.nzz;

		nxy += q.nxy;
		nxz += q.nxz;
		nyz += q.nyz;

		dnx += q.dnx;
		dny += q.dny;
		dnz += q.dnz;

		d2 += q.d2;

		for (uint32 i = 0; i < NumAttributes; i++)
		{
			g[i][0] += q.g[i][0];
			g[i][1] += q.g[i][1];
			g[i][2] += q.g[i][2];
			d[i] += q.d[i];
		}

		a += q.a;

#if PROXYLOD_VOLUME_CONSTRAINT
		nvx += q.nvx;
		nvy += q.nvy;
		nvz += q.nvz;
		dv += q.dv;
#endif

		return *this;
	}

	template< uint32 NumAttributes >
	inline float TQuadricAttr< NumAttributes >::Evaluate(const FVector& Point, const float* Attributes, const float* AttributeWeights) const
	{
		// Q(v) = vt*A*v + 2*bt*v + c

		// v = [ p ]
		//     [ s ]

		// A = [ C  B  ]
		//     [ Bt aI ]

		// C = n*nt
		// B = -g[ 0 .. m ]

		// b = [  dn         ]
		//     [ -d[ 0 .. m] ]

		// c = d2

		double px = Point.X;
		double py = Point.Y;
		double pz = Point.Z;

		double s[NumAttributes];
		for (uint32 i = 0; i < NumAttributes; i++)
		{
			s[i] = AttributeWeights[i] * Attributes[i];
		}

		// A*v = [ C*p  + B*s ]
		//       [ Bt*p + a*s ]

		// C*p
		double x = px * nxx + py * nxy + pz * nxz;
		double y = px * nxy + py * nyy + pz * nyz;
		double z = px * nxz + py * nyz + pz * nzz;

		// B*s
		for (uint32 i = 0; i < NumAttributes; i++)
		{
			x -= g[i][0] * s[i];
			y -= g[i][1] * s[i];
			z -= g[i][2] * s[i];
		}

		// vt*A*v = pt * ( C*p + B*s ) + st * ( Bt*p + a*s )

		// pt * (C*p + B*s)
		double vAv = px * x + py * y + pz * z;

		// st * ( Bt*p + a*s )
		for (uint32 i = 0; i < NumAttributes; i++)
		{
			vAv += s[i] * (a * s[i] - g[i][0] * px - g[i][1] * py - g[i][2] * pz);
		}

		// bt*v
		double btv = px * dnx + py * dny + pz * dnz;
		for (uint32 i = 0; i < NumAttributes; i++)
		{
			btv -= d[i] * s[i];
		}

		// Q(v) = vt*A*v + 2*bt*v + c
		double Q = vAv + 2.0 * btv + d2;

		//check( Q > -1.0 );
		if (!FMath::IsFinite(Q))
		{
			UE_LOGF(LogProxyQuadric, Warning, "Quadric point+attributes evaluate detected possible degenerate. Returning 0.");
			Q = 0;
		}

		return Q;
	}

	template< uint32 NumAttributes >
	inline void TQuadricAttr< NumAttributes >::CalcAttributes(const FVector& Point, float* Attributes, const float* AttributeWeights) const
	{
		double px = Point.X;
		double py = Point.Y;
		double pz = Point.Z;

		for (uint32 i = 0; i < NumAttributes; i++)
		{
			if (AttributeWeights[i] != 0.0f)
			{
				double s = g[i][0] * px + g[i][1] * py + g[i][2] * pz + d[i];
				Attributes[i] = s / (a * AttributeWeights[i]);
			}
			else
			{
				Attributes[i] = 0.0f;
			}
		}
	}


	template< uint32 NumAttributes >
	class TQuadricAttrOptimizer
	{
	public:
		TQuadricAttrOptimizer();

		void		AddQuadric(const FQuadric& q);
		void		AddQuadric(const TQuadricAttr< NumAttributes >& q);

		// Find optimal point for minimal error
		bool		Optimize(FVector& p) const;

	private:
		double		nxx;
		double		nyy;
		double		nzz;

		double		nxy;
		double		nxz;
		double		nyz;

		double		dnx;
		double		dny;
		double		dnz;

		double		a;

#if PROXYLOD_VOLUME_CONSTRAINT
		double		nvx;
		double		nvy;
		double		nvz;
		double		dv;
#endif

		double		BBtxx;
		double		BBtyy;
		double		BBtzz;
		double		BBtxy;
		double		BBtxz;
		double		BBtyz;

		double		Bdx;
		double		Bdy;
		double		Bdz;
	};

	template< uint32 NumAttributes >
	inline TQuadricAttrOptimizer< NumAttributes >::TQuadricAttrOptimizer()
	{
		nxx = 0.0;
		nyy = 0.0;
		nzz = 0.0;

		nxy = 0.0;
		nxz = 0.0;
		nyz = 0.0;

		dnx = 0.0;
		dny = 0.0;
		dnz = 0.0;

		a = 0.0;

#if PROXYLOD_VOLUME_CONSTRAINT
		nvx = 0.0;
		nvy = 0.0;
		nvz = 0.0;
		dv = 0.0;
#endif

		BBtxx = 0.0;
		BBtyy = 0.0;
		BBtzz = 0.0;
		BBtxy = 0.0;
		BBtxz = 0.0;
		BBtyz = 0.0;

		Bdx = 0.0;
		Bdy = 0.0;
		Bdz = 0.0;
	}

	template< uint32 NumAttributes >
	inline void TQuadricAttrOptimizer< NumAttributes >::AddQuadric(const FQuadric& q)
	{
		nxx += q.nxx;
		nyy += q.nyy;
		nzz += q.nzz;

		nxy += q.nxy;
		nxz += q.nxz;
		nyz += q.nyz;

		dnx += q.dnx;
		dny += q.dny;
		dnz += q.dnz;

		a += q.a;
	}

	template< uint32 NumAttributes >
	inline void TQuadricAttrOptimizer< NumAttributes >::AddQuadric(const TQuadricAttr< NumAttributes >& q)
	{
		nxx += q.nxx;
		nyy += q.nyy;
		nzz += q.nzz;

		nxy += q.nxy;
		nxz += q.nxz;
		nyz += q.nyz;

		dnx += q.dnx;
		dny += q.dny;
		dnz += q.dnz;

		a += q.a;

#if PROXYLOD_VOLUME_CONSTRAINT
		nvx += q.nvx;
		nvy += q.nvy;
		nvz += q.nvz;
		dv += q.dv;
#endif

		// B*Bt
		for (uint32 i = 0; i < NumAttributes; i++)
		{
			BBtxx += q.g[i][0] * q.g[i][0];
			BBtyy += q.g[i][1] * q.g[i][1];
			BBtzz += q.g[i][2] * q.g[i][2];

			BBtxy += q.g[i][0] * q.g[i][1];
			BBtxz += q.g[i][0] * q.g[i][2];
			BBtyz += q.g[i][1] * q.g[i][2];
		}

		// -B*d
		for (int i = 0; i < NumAttributes; i++)
		{
			Bdx += q.g[i][0] * q.d[i];
			Bdy += q.g[i][1] * q.d[i];
			Bdz += q.g[i][2] * q.d[i];
		}
	}

	template< uint32 NumAttributes >
	inline bool TQuadricAttrOptimizer< NumAttributes >::Optimize(FVector& Point) const
	{
		// A * v = -b

		// v = [ p ]
		//     [ s ]

		// A = [ C  B  ]
		//     [ Bt aI ]

		// C = n*nt
		// B = -g[ 0 .. m ]

		// b = [  dn         ]
		//     [ -d[ 0 .. m] ]

		// ( C - 1/a * B*Bt ) * p = -1/a * B*d - dn

		checkSlow(a != 0.0);

		// M = C - 1/a * B*Bt
		double ia = 1.0 / a;
		double Mxx = nxx - ia * BBtxx;
		double Myy = nyy - ia * BBtyy;
		double Mzz = nzz - ia * BBtzz;

		double Mxy = nxy - ia * BBtxy;
		double Mxz = nxz - ia * BBtxz;
		double Myz = nyz - ia * BBtyz;

		// -1/a * B*d - dn
		double aBddnx = ia * Bdx - dnx;
		double aBddny = ia * Bdy - dny;
		double aBddnz = ia * Bdz - dnz;

#if PROXYLOD_VOLUME_CONSTRAINT
		// Only use the volume constraint if it is well conditioned
		if (nvx*nvx + nvy*nvy + nvz*nvz > 1e-8)
		{
			/*
			float4x4 M =
			{
			Mxx, Mxy, Mxz, nvx,
			Mxy, Myy, Myz, nvy,
			Mxz, Myz, Mzz, nvz,
			nvx, nvy, nvz, 0
			};
			float4 b = { aBddnx, aBddny, aBddnz, -dv };
			p = ( Inverse(M) * b ).xyz;
			*/

			// 4x4 matrix inverse

			// 2x2 sub-determinants required to calculate 4x4 determinant
			double det2_01_01 = Mxx * Myy - Mxy * Mxy;
			double det2_01_02 = Mxx * Myz - Mxz * Mxy;
			double det2_01_12 = Mxy * Myz - Mxz * Myy;
			double det2_01_03 = Mxx * nvy - nvx * Mxy;
			double det2_01_13 = Mxy * nvy - nvx * Myy;
			double det2_01_23 = Mxz * nvy - nvx * Myz;

			// 3x3 sub-determinants required to calculate 4x4 determinant
			double iNvx = Mzz * det2_01_13 - Myz * det2_01_23 - nvz * det2_01_12;
			double iNvy = Mxz * det2_01_23 - Mzz * det2_01_03 + nvz * det2_01_02;
			double iNvz = Myz * det2_01_03 - Mxz * det2_01_13 - nvz * det2_01_01;

			double det = iNvx * nvx + iNvy * nvy + iNvz * nvz;

			if (FMath::Abs(det) < 1e-8)
			{
				return false;
			}

			double invDet = 1.0 / det;

			// remaining 2x2 sub-determinants
			double det2_03_02 = Mxx * nvz - Mxz * nvx;
			double det2_03_12 = Mxy * nvz - Mxz * nvy;
			double det2_13_12 = Myy * nvz - Myz * nvy;

			double det2_03_03 = -nvx * nvx;
			double det2_03_13 = -nvx * nvy;
			double det2_03_23 = -nvx * nvz;

			double det2_13_13 = -nvy * nvy;
			double det2_13_23 = -nvy * nvz;

			// remaining 3x3 sub-determinants
			double iMxx = Mzz * det2_13_13 - Myz * det2_13_23 - nvz * det2_13_12;
			double iMxy = Myz * det2_03_23 - Mzz * det2_03_13 + nvz * det2_03_12;
			double iMyy = Mzz * det2_03_03 - Mxz * det2_03_23 - nvz * det2_03_02;

			double iMxz = nvy * det2_01_23 - nvz * det2_01_13;
			double iMyz = nvz * det2_01_03 - nvx * det2_01_23;
			double iMzz = nvx * det2_01_13 - nvy * det2_01_03;

			Point.X = invDet * (aBddnx * iMxx + aBddny * iMxy + aBddnz * iMxz - iNvx * dv);
			Point.Y = invDet * (aBddnx * iMxy + aBddny * iMyy + aBddnz * iMyz - iNvy * dv);
			Point.Z = invDet * (aBddnx * iMxz + aBddny * iMyz + aBddnz * iMzz - iNvz * dv);
		}
		else
#endif
		{
			/*
			float3x3 M =
			{
			Mxx, Mxy, Mxz,
			Mxy, Myy, Myz,
			Mxz, Myz, Mzz
			};
			float3 b = { aBddnx, aBddny, aBddnz };
			p = Inverse(M) * b;
			*/

			double iMxx = Myy * Mzz - Myz * Myz;
			double iMxy = Mxz * Myz - Mzz * Mxy;
			double iMxz = Mxy * Myz - Myy * Mxz;

			double det = Mxx * iMxx + Mxy * iMxy + Mxz * iMxz;

			if (FMath::Abs(det) < 1e-8)
			{
				return false;
			}

			double invDet = 1.0 / det;

			double iMyy = Mxx * Mzz - Mxz * Mxz;
			double iMyz = Mxy * Mxz - Mxx * Myz;
			double iMzz = Mxx * Myy - Mxy * Mxy;

			Point.X = invDet * (aBddnx * iMxx + aBddny * iMxy + aBddnz * iMxz);
			Point.Y = invDet * (aBddnx * iMxy + aBddny * iMyy + aBddnz * iMyz);
			Point.Z = invDet * (aBddnx * iMxz + aBddny * iMyz + aBddnz * iMzz);
		}

		return true;
	}
}