// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODRasterizer.h"

#include "ProxyLODBarycentricUtilities.h"
#include "ProxyLODMeshUtilities.h"
#include "ProxyLODThreadedWrappers.h"

THIRD_PARTY_INCLUDES_START
#include <tbb/spin_mutex.h>
THIRD_PARTY_INCLUDES_END

namespace ProxyLOD
{



	FRasterGrid::Ptr RasterizeTriangles(const FVertexDataMesh& TriangleMesh, const FTextureAtlasDesc& TextureAtlasDesc, const int32 Padding, const int32 SuperSampleCount)
	{

		typedef std::array<int32, 2>  IntArray2;

		// Number of super samples in each direction

		const int32 SuperSamples = FMath::CeilToInt(FMath::Sqrt(float(SuperSampleCount)));

		const FIntPoint& TextureSize = TextureAtlasDesc.Size;

		const double Lx = TextureSize.X * SuperSamples;
		const double Ly = TextureSize.Y * SuperSamples;

		// Create the target grid.

		FRasterGrid::Ptr RasterGridPtr = FRasterGrid::Create(TextureSize.X * SuperSamples, TextureSize.Y * SuperSamples);
		FIntPoint GridSize = RasterGridPtr->Size();
		FRasterGrid& RasterGrid = *RasterGridPtr;

		// The grid spacing used when converting from texel space to UV space

		const double DeltaX = 1. / Lx;
		const double DeltaY = 1. / Ly;

		// This mesh only hold triangles: 3 indices per triangle.

		const auto& UVArray = TriangleMesh.UVs;
		const auto& Indices = TriangleMesh.Indices;
		const uint32 NumTriangles = Indices.Num() / 3;

		TArray<float> TriangleAreaSqrArray;
		ResizeArray(TriangleAreaSqrArray, NumTriangles);

		Parallel_For(FIntRange(0, NumTriangles),
			[&TriangleMesh, &TriangleAreaSqrArray](const FIntRange& Range)
		{
			const auto& Position = TriangleMesh.Points;
			const auto& Indices = TriangleMesh.Indices;

			for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				int32 offset = i * 3;
				const FVector E10 = FVector(Position[Indices[offset + 1]] - Position[Indices[offset]]);
				const FVector E20 = FVector(Position[Indices[offset + 2]] - Position[Indices[offset]]);

				const FVector Wedge = FVector::CrossProduct(E10, E20);

				// This is really 4 * the area squared.
				TriangleAreaSqrArray[i] = Wedge.SizeSquared();
			}

		});

		// Used to avoid data races.
		TGrid<tbb::spin_mutex>  MutexGrid(Lx, Ly);

		Parallel_For(FUIntRange(0, NumTriangles),
			[&](const FUIntRange& Range)
		{
			for (uint32 TriangleId = Range.begin(); TriangleId < Range.end(); ++TriangleId)
			{

				// Skip degenerate triangles.  I think these were added by the UV
				// generation.

				const bool bDegenerate = (TriangleAreaSqrArray[TriangleId] < 1.e-6);
				if (bDegenerate) continue;

				// Get the vertices
				DTriangle2d TriangleUV;
				for (uint32 V = 0, Idx = 3 * TriangleId; V < 3; ++V)
				{
					uint32 VertexId = Indices[Idx + V];
					TriangleUV[V] = { UVArray[VertexId][0], UVArray[VertexId][1] };
				}


				// Compute the bounding box for this triangle in Texel space
				IntArray2 MinTexel;
				IntArray2 MaxTexel;
				{

					// Compute the Bounding Box for this triangle in UV space  

					DVec2d  MinUV;
					DVec2d  MaxUV;
					ComputeBBox(TriangleUV, MinUV, MaxUV);



					// Convert to Texture space  Bounding Box 

					MinTexel[0] = FMath::FloorToInt(MinUV[0] * Lx) - Padding;    MinTexel[1] = FMath::FloorToInt(MinUV[1] * Ly) - Padding;
					MaxTexel[0] = FMath::CeilToInt(MaxUV[0] * Lx) + Padding;     MaxTexel[1] = FMath::CeilToInt(MaxUV[1] * Ly) + Padding;

					// Clip against the texture size

					MinTexel[0] = FMath::Max(MinTexel[0], 0);               MinTexel[1] = FMath::Max(MinTexel[1], 0);
					MaxTexel[0] = FMath::Min(MaxTexel[0], GridSize.X);      MaxTexel[1] = FMath::Min(MaxTexel[1], GridSize.Y);
				}



				// Iterate over the texel bbox for this triangle

				for (int32 i = MinTexel[0]; i < MaxTexel[0]; ++i)
				{
					for (int32 j = MinTexel[1]; j < MaxTexel[1]; ++j)
					{


						// The texel

						FRasterGrid::ValueType& TexelData = RasterGrid(i, j);

						// This texel has already been marked 'inside' a different triangle.
						// Implicit assumption: no triangles overlap.  
						// NB: what about the case where an edge intersects the texel center?  
						// then it would be 'inside' the two triangles. Currently the first one wins.

						if (TexelData.SignedDistance < 0.) continue;

						// This texel is [i, i+1) x [j, j+1) 
						// The UV of this texel center

						const DVec2d TexelCenterUV = { DeltaX * (i + 0.5), DeltaY * (j + 0.5) };
						//const DVec2d TexelCenterUV = { DeltaX * (i), DeltaY * (j) };
						// Compute the Scaled Distances for a quick inside / outside test applied to the texel center.

						DArray3d ScaledDistances;
						ComputeScaledDistances(TriangleUV, TexelCenterUV, ScaledDistances);
						//ConvertScaledDistanceToBarycentric(TriangleUV, ScaledDistances);

						// This assumes the polygon was given in a counter clockwise orientation.   
						// If the orientation were clockwise, these would all become <= instead.


						const bool bInside = (ScaledDistances[0] >= 0. && ScaledDistances[1] >= 0. && ScaledDistances[2] >= 0.);

						{
							// Only one thread is allowed to compute data for this texel at a time..

							tbb::spin_mutex::scoped_lock(MutexGrid(i, j));

							// verify that another thread hasn't already identified this texel as interior to a triangle.

							if (TexelData.SignedDistance < 0.) continue;


							if (bInside)
							{
								// Store interior information in the texel

								ConvertScaledDistanceToBarycentric(TriangleUV, ScaledDistances);

								TexelData.TriangleId = TriangleId;
								TexelData.SignedDistance = -1.;
								TexelData.BarycentricCoords = ScaledDistances;
							}
							else
							{
								// Outside the triangle, but we compute the signed distance and record triangle data
								// if this is the closest triangle so far

								DVec2d ClosestPointUV; // Point in UV space projected onto closest point on the triangle
								double DstSqr = std::numeric_limits<double>::max();
								SqrDistanceToSegment(TriangleUV[0], TriangleUV[1], TexelCenterUV, ClosestPointUV, DstSqr);
								SqrDistanceToSegment(TriangleUV[1], TriangleUV[2], TexelCenterUV, ClosestPointUV, DstSqr);
								SqrDistanceToSegment(TriangleUV[2], TriangleUV[0], TexelCenterUV, ClosestPointUV, DstSqr);

								double Dst = FMath::Sqrt(DstSqr);

								// Are we closer than any previous triangle to this texel?
								// If so, update with info from this triangle.

								if (Dst < TexelData.SignedDistance)
								{
									// Compute the barycentric coords of the closest point on the triangle.
									// NB: at least one of the coords should be zero since the point lies on 
									// the boundary of the triangle
									ComputeScaledDistances(TriangleUV, ClosestPointUV, ScaledDistances);
									ConvertScaledDistanceToBarycentric(TriangleUV, ScaledDistances);

									TexelData.TriangleId = TriangleId;
									TexelData.SignedDistance = Dst;
									TexelData.BarycentricCoords = ScaledDistances;
								}

							}// in vs out
						} // scope lock
					}// j
				}//i
			} // triangle
		}); // parallel

		return RasterGridPtr;
	}

	static void InterpolateWedgeColors(const FMeshDescription& RawMesh, const ProxyLOD::FRasterGrid& DstUVGrid, FFlattenMaterial& OutMaterial)
	{
		TArray<FColor>& ColorBuffer = OutMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse);
		FIntPoint Size = OutMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse);
		ResizeArray(ColorBuffer, Size.X * Size.Y);

		TArrayView<const FVector4f> VertexInstanceColors = FStaticMeshConstAttributes(RawMesh).GetVertexInstanceColors().GetRawArray();

		for (int j = 0; j < Size.Y; ++j)
		{
			for (int i = 0; i < Size.X; ++i)
			{
				const auto& Data = DstUVGrid(i, j);
				int32 TriangleId = Data.TriangleId;

				if (TriangleId > -1 || Data.SignedDistance < 0.)
				{
					const auto& BarycentericCoords = Data.BarycentricCoords;

					const FVertexInstanceID Indexes[3] = { FVertexInstanceID(TriangleId * 3), FVertexInstanceID(TriangleId * 3 + 1), FVertexInstanceID(TriangleId * 3 + 2) };
					FLinearColor ColorSamples[3];
					ColorSamples[0] = FLinearColor(VertexInstanceColors[Indexes[0]]);
					ColorSamples[1] = FLinearColor(VertexInstanceColors[Indexes[1]]);
					ColorSamples[2] = FLinearColor(VertexInstanceColors[Indexes[2]]);
					FLinearColor AvgColor = ProxyLOD::InterpolateVertexData(BarycentericCoords, ColorSamples);

					ColorBuffer[i + j * Size.X] = AvgColor.ToFColor(true);
				}
				else
				{
					ColorBuffer[i + j * Size.X] = FColor(0, 0, 0);
				}
			}
		}

	};

	void DebugVertexAndTextureColors(FMeshDescription& RawMesh, const ProxyLOD::FRasterGrid& DstUVGrid, FFlattenMaterial& OutMaterial)
	{
		// Testing the Barycentric interpolation.

		AddWedgeColors(RawMesh);

		InterpolateWedgeColors(RawMesh, DstUVGrid, OutMaterial);
	};


	// Returns 'true' if the dilation was needed.
	// if 'false' is returned, no dilation was actually done.
	// Note: ValueType must support ForceInitToZero

	template <typename GridOrWrapperType>
	bool TDilateGrid(GridOrWrapperType& RasterGrid, TGrid<int32>& TopologyGrid)
	{
		checkSlow(TopologyGrid.Size() == RasterGrid.Size());

		typedef typename GridOrWrapperType::ValueType   ValueType;

		const FIntPoint Size = RasterGrid.Size();
		const int32 TexelCount = Size.X * Size.Y;


		// Count the number of texels that are full.
		int32 FullCount =
			Parallel_Reduce(FIntRange(0, TexelCount), 0, [&TopologyGrid](const FIntRange& Range, const int32& InSum)->int32
		{
			const int32* TopologyData = TopologyGrid.GetData();
			int32 LocalSum = 0;
			for (int32 r = Range.begin(), R = Range.end(); r < R; ++r)
			{
				LocalSum += TopologyData[r];
			}

			return LocalSum + InSum;
		},
				[](int32 SumA, int32 SumB)->int32
		{
			return SumA + SumB;
		});

		int32 InActiveCount = TexelCount - FullCount;

		// return false if this function did no work
		if (InActiveCount == 0) return false;


		// Create a iteration grid as deep copy of the topology grid.

		TGrid<int32> IterationGrid(Size.X, Size.Y);

		// copy the topology into the iteration grid.
		Parallel_For(FIntRange(0, Size.X * Size.Y),
			[&TopologyGrid, &IterationGrid](const FIntRange& Range)
		{
			const int32* TopologyData = TopologyGrid.GetData();
			int32* IterationData = IterationGrid.GetData();

			for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				IterationData[i] = TopologyData[i];
			}

		});

		const auto& DotColor = [](const int32(&Mask)[4], const ValueType(&Samples)[4])->ValueType
		{
			ValueType Result = float(Mask[0]) * Samples[0] + float(Mask[1]) * Samples[1] + float(Mask[2]) * Samples[2] + float(Mask[3]) * Samples[3];
			return Result;
		};


		// Do the dilation of the interior
		Parallel_For(FIntRange2d(1, Size.X - 1, 1, Size.Y - 1),
			[&IterationGrid, &TopologyGrid, &RasterGrid, &DotColor](const FIntRange2d& Range)
		{

			for (int i = Range.rows().begin(), I = Range.rows().end(); i < I; ++i)
			{
				for (int j = Range.cols().begin(), J = Range.cols().end(); j < J; ++j)
				{


					if (IterationGrid(i, j) == 0)
					{
						// probe all the neighbors
						const int32 Mask[4] = { IterationGrid(i - 1, j), IterationGrid(i + 1, j), IterationGrid(i, j - 1), IterationGrid(i, j + 1) };
						const int32 Weight = Mask[0] + Mask[1] + Mask[2] + Mask[3];

						if (Weight != 0)
						{
							const ValueType ValueSamples[4] = { RasterGrid(i - 1, j), RasterGrid(i + 1, j), RasterGrid(i, j - 1), RasterGrid(i, j + 1) };
							const ValueType AverageValue = DotColor(Mask, ValueSamples) / float(Weight);

							RasterGrid(i, j) = AverageValue;

							// mark this texel as occupied for the next iteration of dilate()
							TopologyGrid(i, j) = 1;
						}
					}
				}
			}
		});

		// Do the top and bottom.  Note this assumes Size.Y - 1 > 0
		Parallel_For(FIntRange(1, Size.X - 1),
			[&IterationGrid, &TopologyGrid, &RasterGrid, &DotColor](const FIntRange& Range)
		{
			{
				// do the top
				const int32 j = 0;

				for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
				{

					if (IterationGrid(i, j) == 0)
					{
						// probe all the neighbors
						const int32 Mask[4] = { IterationGrid(i - 1, j), IterationGrid(i + 1, j), 0, IterationGrid(i, j + 1) };
						const int32 Weight = Mask[0] + Mask[1] + Mask[2] + Mask[3];

						if (Weight != 0)
						{
							const ValueType ValueSamples[4] = { RasterGrid(i - 1, j), RasterGrid(i + 1, j), ValueType(ForceInitToZero), RasterGrid(i, j + 1) };
							const ValueType AverageValue = DotColor(Mask, ValueSamples) / float(Weight);

							RasterGrid(i, j) = AverageValue;

							// mark this texel as occupied for the next iteration of dilate()
							TopologyGrid(i, j) = 1;
						}
					}
				}
			}
		});

		Parallel_For(FIntRange(1, Size.X - 1),
			[&IterationGrid, &TopologyGrid, &RasterGrid, &Size, &DotColor](const FIntRange& Range)
		{
			{
				// do the bottom
				const int32 j = Size.Y - 1;

				for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
				{
					// this is an empty cell

					ValueType Result(ForceInitToZero);
					if (IterationGrid(i, j) == 0)
					{
						// probe all the neighbors
						const int32 Mask[4] = { IterationGrid(i - 1, j), IterationGrid(i + 1, j), IterationGrid(i, j - 1), 0 };
						const int32 Weight = Mask[0] + Mask[1] + Mask[2] + Mask[3];

						if (Weight != 0)
						{
							const ValueType ValueSamples[4] = { RasterGrid(i - 1, j), RasterGrid(i + 1, j),  RasterGrid(i, j - 1), ValueType(ForceInitToZero) };
							const ValueType AverageValue = DotColor(Mask, ValueSamples) / float(Weight);

							RasterGrid(i, j) = AverageValue;

							// mark this texel as occupied for the next iteration of dilate()
							TopologyGrid(i, j) = 1;
						}
					}
				}
			}
		});


		Parallel_For(FIntRange(1, Size.Y - 1),
			[&IterationGrid, &TopologyGrid, &RasterGrid, &DotColor](const FIntRange& Range)
		{

			{
				// do the left side
				const int32 i = 0;
				for (int32 j = Range.begin(), J = Range.end(); j < J; ++j)
				{
					if (IterationGrid(i, j) == 0)
					{
						// probe all the neighbors
						const int32 Mask[4] = { 0, IterationGrid(i + 1, j), IterationGrid(i, j - 1), IterationGrid(i, j + 1) };
						const int32 Weight = Mask[0] + Mask[1] + Mask[2] + Mask[3];

						if (Weight != 0)
						{
							const ValueType ValueSamples[4] = { ValueType(ForceInitToZero), RasterGrid(i + 1, j), RasterGrid(i, j - 1), RasterGrid(i, j + 1) };
							const ValueType AverageValue = DotColor(Mask, ValueSamples) / float(Weight);

							RasterGrid(i, j) = AverageValue;

							// mark this texel as occupied for the next iteration of dilate()
							TopologyGrid(i, j) = 1;
						}
					}

				}

			}
		});
		Parallel_For(FIntRange(1, Size.Y - 1),
			[&IterationGrid, &TopologyGrid, &RasterGrid, &Size, &DotColor](const FIntRange& Range)
		{
			{
				// do the right side
				const int32 i = Size.X - 1;
				for (int32 j = Range.begin(), J = Range.end(); j < J; ++j)
				{
					if (IterationGrid(i, j) == 0)
					{
						// probe all the neighbors
						const int32 Mask[4] = { IterationGrid(i - 1, j), 0, IterationGrid(i, j - 1), IterationGrid(i, j + 1) };
						const int32 Weight = Mask[0] + Mask[1] + Mask[2] + Mask[3];

						if (Weight != 0)
						{
							const ValueType ValueSamples[4] = { RasterGrid(i - 1, j), ValueType(ForceInitToZero), RasterGrid(i, j - 1), RasterGrid(i, j + 1) };
							const ValueType AverageValue = DotColor(Mask, ValueSamples) / float(Weight);

							RasterGrid(i, j) = AverageValue;

							// mark this texel as occupied for the next iteration of dilate()
							TopologyGrid(i, j) = 1;
						}
					}

				}
			}
		});


		// do the corners
		{
			int32 i = 0;
			int32 j = 0;
			// this is an empty cell
			if (IterationGrid(i, j) == 0)
			{
				const int32 Mask[4] = { 0,  IterationGrid(i + 1, j) , 0, IterationGrid(i, j + 1) };
				const int32 Weight = Mask[0] + Mask[1] + Mask[2] + Mask[3];

				if (Weight != 0)
				{
					const ValueType ValueSamples[4] = { ValueType(ForceInitToZero), RasterGrid(i + 1, j), ValueType(ForceInitToZero), RasterGrid(i, j + 1) };
					const ValueType AverageValue = DotColor(Mask, ValueSamples) / float(Weight);

					RasterGrid(i, j) = AverageValue;

					// mark this texel as occupied for the next iteration of dilate()
					TopologyGrid(i, j) = 1;
				}
			}

			i = Size.X - 1;
			j = 0;
			// this is an empty cell
			if (IterationGrid(i, j) == 0)
			{
				// probe all the neighbors
				const int32 Mask[4] = { IterationGrid(i - 1, j), 0, 0, IterationGrid(i, j + 1) };
				const int32 Weight = Mask[0] + Mask[1] + Mask[2] + Mask[3];

				if (Weight != 0)
				{
					const ValueType ValueSamples[4] = { RasterGrid(i - 1, j), ValueType(ForceInitToZero), ValueType(ForceInitToZero), RasterGrid(i, j + 1) };
					const ValueType AverageValue = DotColor(Mask, ValueSamples) / float(Weight);

					RasterGrid(i, j) = AverageValue;

					// mark this texel as occupied for the next iteration of dilate()
					TopologyGrid(i, j) = 1;
				}
			}

			i = Size.X - 1;
			j = Size.Y - 1;
			// this is an empty cell
			if (IterationGrid(i, j) == 0)
			{
				// probe all the neighbors
				const int32 Mask[4] = { IterationGrid(i - 1, j), 0, IterationGrid(i, j - 1), 0 };
				const int32 Weight = Mask[0] + Mask[1] + Mask[2] + Mask[3];

				if (Weight != 0)
				{
					const ValueType ValueSamples[4] = { RasterGrid(i - 1, j), ValueType(ForceInitToZero), RasterGrid(i, j - 1), ValueType(ForceInitToZero) };
					const ValueType AverageValue = DotColor(Mask, ValueSamples) / float(Weight);

					RasterGrid(i, j) = AverageValue;

					// mark this texel as occupied for the next iteration of dilate()
					TopologyGrid(i, j) = 1;
				}
			}

			i = 0;
			j = Size.Y - 1;
			if (IterationGrid(i, j) == 0)
			{
				// probe all the neighbors
				const int32 Mask[4] = { 0, IterationGrid(i + 1, j), IterationGrid(i, j - 1), 0 };
				const int32 Weight = Mask[0] + Mask[1] + Mask[2] + Mask[3];

				if (Weight != 0)
				{
					const ValueType ValueSamples[4] = { ValueType(ForceInitToZero), RasterGrid(i + 1, j), RasterGrid(i, j - 1), ValueType(ForceInitToZero) };
					const ValueType AverageValue = DotColor(Mask, ValueSamples) / float(Weight);

					RasterGrid(i, j) = AverageValue;

					// mark this texel as occupied for the next iteration of dilate()
					TopologyGrid(i, j) = 1;
				}
			}
		}

		// did some work
		return true;
	}
}

bool ProxyLOD::DilateGrid(FLinearColorGrid& RasterGrid, TGrid<int32>& TopologyGrid)
{
	return TDilateGrid(RasterGrid, TopologyGrid);
}