// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// UE 
#include "Math/Vector.h"
#include "Math/UnrealMathUtility.h" // ComputeBaryCentric2D

// std
THIRD_PARTY_INCLUDES_START
#include <limits>
#include <array>
#include <tuple>
THIRD_PARTY_INCLUDES_END

// openvdb
//#include <openvdb/openvdb.h>


// ProxyLOD pluggin


#include "ProxyLODBarycentricUtilities.h"
#include "ProxyLODGrid2d.h"
#include "ProxyLODMeshTypes.h"
#include "ProxyLODTwoDTriangleUtilities.h"

#include "MaterialUtilities.h"


// * Used to map texels (texture space) to triangles and UV space.
// *  Rasterize 2D triangles into custom data structure. 

namespace ProxyLOD
{

	/**
	*  Class to be used as a per-texel data type.
	*
	*  A grid of this data type is used to map locations (i.e. texels) in the texture atlas
	*  to positions on the corresponding mesh.
	*    
	*  Records TriangleID, Signed Distance and Barycentric Coords.
	*
	*/
	class FRasterizerData
	{
	public:

		FRasterizerData() { }

		FRasterizerData(const FRasterizerData& other) :
			SignedDistance(other.SignedDistance),
			TriangleId(other.TriangleId)
		{
			BarycentricCoords = other.BarycentricCoords;
		}


		double         SignedDistance = std::numeric_limits<double>::max();
		int32          TriangleId  = -1;
		
		// In the space of the 2d triangle defined in texture coord space 
		
		DArray3d   BarycentricCoords = { 0., 0., 0. };
	};

	/**
    *  A grid used to map locations (i.e. texels) in the texture atlas
	*  to positions on the corresponding mesh.
	*
	*  By convention, those texels that fall outside of all UV space triangles will store the
	*  Id of the closest triangle and the distance to that triangle.
	*
	*/
	typedef TGrid<FRasterizerData>      FRasterGrid;

	/**
	* Raterize the UVs from the input TriangleMesh into a 2d grid.
	* The cells of the resulting grid may be put in correspondence with texels
	* in a texture.
	*
	* @param TriangleMesh      Mesh with well defined UVs ( TriangleMesh.UVs )
	* @param TextureAtlasDesc  A description of the texture atlas used in the proxylod pipeline
	*                          Primarily used to define the shape (IMax, JMax) of the resulting 2d Grid.
	* @param Padding           In grid cells.  The bounding box for a UV triangle can be padded when 
	*                          rasterizing the triangle into the 2d grid.
	* @param SuperSampleCount  Scales the TextureAtlas.Size to produce the RasterGrid.Size().
	*                          E.g. if   TextureAtlas.Size = 10,10 and SuperSampleCount = 4
	*                               then RasterGid.Size() = 20,20 
	*
	* @return Two-Dimensional grid of FRasterizerData used to map grid cells to mesh triangles and UV-barycenteric coords within the triangle. 
	*/
	FRasterGrid::Ptr RasterizeTriangles(const FVertexDataMesh& TriangleMesh, 
		                                const FTextureAtlasDesc& TextureAtlasDesc, 
		                                const int32 Padding, 
		                                const int32 SuperSampleCount = 1);


	/**
	* A Debug function that adds wedge colors to the input RawMesh and uses the texel map for the UVs of that RawMesh 
	* to make a material that holds the Barycentric-averaged vertex colors at each texel.
	* It is assumed that UVGrid is the restult of rasterizing the UV triangles of the raw mesh.
	*
	*  @param InOutRawMesh  Mesh with UVs - on return, wedge colors will have been added to the mesh
	*  @param InUVGrid      Rasterization of the UV triangles of the InOutGrid.
	*  @param OutMaterial   Material that, on return has the Barycenteric-averaged vertex colors. 
	*/
	void DebugVertexAndTextureColors(FMeshDescription& InOutRawMesh, const ProxyLOD::FRasterGrid& InUVGrid, FFlattenMaterial& OutMaterial);
	
	/**
	* Two dimensional grid access to a TArray holding FLinearColor.
	*/
	typedef TGrid<FLinearColor>  FLinearColorGrid;
	
	/**
	* This function performs a single step of color dilation, where previously marked "valid" cells in the input
	* RasterGrid allow their values to spread to neighboring, "invalid" cells.
	*
	* On return, previously invalid cells that had at least one valid neighbor will be filled with
	* the average value of valid neighbors and the Topology grid will be updated to indicate the new (valid)
	* state of such cells.
	*  TopologyGrid(i, j) .ne. 0 then RasterGrid(i, j) is assumed to have a valid value.
	*
	* NB:  @param InOutToplogyGrid must have the same shape as @param InOutRasterGrid
	* 
	* @param InOutRasterGrid     A 2d grid of linear colors, with valid values in a subset of grid cells.
	*                           
	* @param InOutTopologyGrid   Grid that implicitly labels RasterGrid cells as valid or invalid.
	*
	* 
	* @return Bool value indicating if any grid cells were actually updated during the Dilate
	*/
	bool DilateGrid(FLinearColorGrid& InOutRasterGrid, TGrid<int32>& InOutTopologyGrid);


} // end namespace
