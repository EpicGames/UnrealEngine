// Copyright (C) 2009 Nine Realms, Inc
//

#include "ProxyLODSimplifier.h"

#include "ProxyLODQuadric.h"

/**
*  Slightly modified version of the quadric simplifier found in MeshSimplifier\Private\MeshSimplify.h
*  that code caries the copyright--
*/
// Copyright (C) 2009 Nine Realms, Inc

//=============
// FMeshSimplifier
//=============
ProxyLOD::FQuadricMeshSimplifier::FQuadricMeshSimplifier(const MeshVertType* Verts, uint32 NumVerts, const uint32* Indexes, uint32 NumIndexes, float CoAlignmentLimit)
	:
	numSVerts(NumVerts),
	numSTris(NumIndexes / 3),
	numVerts(numSVerts),
	numTris(numSTris),
	coAlignmentLimit(CoAlignmentLimit),
	edgeHash(1 << FMath::Min(16u, FMath::FloorLog2(NumVerts))),
	quadricCache(numSVerts, numSTris)
{
	//	SCOPE_LOG_TIME(TEXT("UE_ProxyLOD_Simplifier_Constructor"), nullptr);
	vertFlagLock = 0;
	triFlagLock = 0;

	for (uint32 i = 0; i < NumAttributes; i++)
	{
		attributeWeights[i] = 1.0f;
	}

	sVerts = new SimpVertType[numSVerts];
	sTris = new SimpTriType[numSTris];

	// The cache uses pointer-based arithmetic based on sVerts and sTris.

	quadricCache.RegisterCache(sVerts, sTris);

	// Deep copy the verts

	Parallel_For(FIntRange(0, NumVerts), [&Verts, this](const FIntRange& Range)
	{
		for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
		{
			sVerts[i].vert = Verts[i];
		}
	});

	// Register the verts with the tris

	Parallel_For(FIntRange(0, numSTris), [&Indexes, this](const FIntRange& Range)
	{
		for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
		{
			for (int j = 0; j < 3; j++)
			{
				sTris[i].verts[j] = &sVerts[Indexes[3 * i + j]];
			}
		}
	});


	// -----------

	// Register each tri with the vert.

	for (int32 i = 0; i < numSTris; ++i)
	{
		for (int32 j = 0; j < 3; ++j)
		{
			sTris[i].verts[j]->adjTris.Add(&sTris[i]);
		}
	}


	// -----------

	// Group the verts that share the same location.

	GroupVerts();

	// Populate the TArray of edges.

	int maxEdgeSize = FMath::Min(3 * numSTris, 3 * numSVerts - 6);
	edges.Empty(maxEdgeSize);
	for (int i = 0; i < numSVerts; i++)
	{
		InitVert(&sVerts[i]);
	}

	// Guessed wrong on num edges. Array was resized so fix up pointers.

	if (edges.Num() > maxEdgeSize)
	{
		Parallel_For(FIntRange(0, edges.Num()), [this](const FIntRange& Range)
		{
			for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				SimpEdgeType& edge = edges[i];
				edge.next = &edge;
				edge.prev = &edge;
			}
		});
	}

	FTaskGroup TaskGroup;

	TaskGroup.Run([&, this]
	{
		GroupEdges();
	});

	TaskGroup.RunAndWait([&, this]
	{
		this->edgeHash.Resize(this->edges.Num());
		{
			TArray<int32> HashValues;
			ResizeArray(HashValues, this->edges.Num());

			Parallel_For(FIntRange(0, this->edges.Num()),
				[&HashValues, this](const FIntRange& Range)
			{
				for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
				{
					HashValues[i] = HashEdge(this->edges[i].v0, this->edges[i].v1);
				}
			});

			for (int i = 0; i < this->edges.Num(); i++)
			{
				this->edgeHash.Add(HashValues[i], i);
			}
		}

		this->edgeHeap.Resize(this->edges.Num(), this->edges.Num());
	});

}



ProxyLOD::FQuadricMeshSimplifier::~FQuadricMeshSimplifier()
{
	delete[] sVerts;
	delete[] sTris;
}


void ProxyLOD::FQuadricMeshSimplifier::SetAttributeWeights(const float* weights)
{
	for (uint32 i = 0; i < NumAttributes; i++)
	{
		attributeWeights[i] = weights[i];
	}
}



void ProxyLOD::FQuadricMeshSimplifier::SetBoundaryLocked()
{
	TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;

	for (int i = 0; i < numSVerts; i++)
	{

		SimpVertType* v0 = &sVerts[i];
		checkSlow(v0 != NULL);
		check(v0->adjTris.Num() > 0);

		adjVerts.Reset();
		v0->FindAdjacentVertsGroup(adjVerts);

		for (SimpVertType* v1 : adjVerts)
		{
			if (v0 < v1)
			{
				LockTriFlags(SIMP_MARK1);

				// set if this edge is boundary
				// find faces that share v0 and v1
				v0->EnableAdjTriFlagsGroup(SIMP_MARK1);
				v1->DisableAdjTriFlagsGroup(SIMP_MARK1);

				int faceCount = 0;
				SimpVertType* vert = v0;
				do
				{
					for (TriIterator j = vert->adjTris.Begin(); j != vert->adjTris.End(); ++j)
					{
						SimpTriType* tri = *j;
						faceCount += tri->TestFlags(SIMP_MARK1) ? 0 : 1;
					}
					vert = vert->next;
				} while (vert != v0);

				v0->DisableAdjTriFlagsGroup(SIMP_MARK1);

				if (faceCount == 1)
				{
					// only one face on this edge
					v0->EnableFlagsGroup(SIMP_LOCKED);
					v1->EnableFlagsGroup(SIMP_LOCKED);
				}

				UnlockTriFlags(SIMP_MARK1);
			}
		}
	}
}

void ProxyLOD::FQuadricMeshSimplifier::InitVert(SimpVertType* v)
{
	checkSlow(v->adjTris.Num() > 0);

	TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;
	v->FindAdjacentVerts(adjVerts);

	SimpVertType* v0 = v;
	for (SimpVertType* v1 : adjVerts)
	{
		if (v0 < v1)
		{
			// add edge
			edges.AddDefaulted();
			SimpEdgeType& edge = edges.Last();
			edge.v0 = v0;
			edge.v1 = v1;
		}
	}
}

void ProxyLOD::FQuadricMeshSimplifier::GroupVerts()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::FQuadricMeshSimplifier::GroupVerts)

	// group verts that share a point
	FHashTable HashTable(1 << FMath::Min(16u, FMath::FloorLog2(numSVerts / 2)), numSVerts);

	TArray<uint32> HashValues;
	ResizeArray(HashValues, numSVerts);
	{
		// Compute the hash values

		Parallel_For(FIntRange(0, numSVerts),
			[&HashValues, this](const FIntRange& Range)
		{
			const auto& Verts = this->sVerts;
			for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
			{
				HashValues[i] = HashPoint((FVector)Verts[i].GetPos());
			}
		}
		);

		// insert the hash values.
		for (int i = 0; i < numSVerts; i++)
		{
			HashTable.Add(HashValues[i], i);
		}
	}

	for (int i = 0; i < numSVerts; i++)
	{
		// already grouped
		if (sVerts[i].next != &sVerts[i])
		{
			continue;
		}

		// find any matching verts
		const uint32 hash = HashValues[i];
		SimpVertType* v1 = &sVerts[i];

		for (int j = HashTable.First(hash); HashTable.IsValid(j); j = HashTable.Next(j))
		{

			SimpVertType* v2 = &sVerts[j];

			if (v1 == v2)
				continue;

			// link
			if (v1->GetPos() == v2->GetPos())
			{
				checkSlow(v2->next == v2);
				checkSlow(v2->prev == v2);

				v2->next = v1->next;
				v2->prev = v1;
				v2->next->prev = v2;
				v2->prev->next = v2;
			}
		}
	}
}

void ProxyLOD::FQuadricMeshSimplifier::GroupEdges()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::FQuadricMeshSimplifier::GroupEdges)

	FHashTable HashTable(1 << FMath::Min(16u, FMath::FloorLog2(edges.Num() / 2)), edges.Num());

	TArray<uint32> HashValues;
	ResizeArray(HashValues, edges.Num());

	Parallel_For(FIntRange(0, edges.Num()),
		[&HashValues, this](const FIntRange& Range)
	{
		const auto& Edges = this->edges;
		for (int32 i = Range.begin(), I = Range.end(); i < I; ++i)
		{
			uint32 Hash0 = HashPoint((FVector)Edges[i].v0->GetPos());
			uint32 Hash1 = HashPoint((FVector)Edges[i].v1->GetPos());
			HashValues[i] = Murmur32({ FMath::Min(Hash0, Hash1), FMath::Max(Hash0, Hash1) });
		}

	});

	for (int i = 0; i < edges.Num(); i++)
	{
		HashTable.Add(HashValues[i], i);
	}

	for (int i = 0; i < edges.Num(); i++)
	{
		// already grouped
		if (edges[i].next != &edges[i])
		{
			continue;
		}

		// find any matching edges
		uint32 Hash = HashValues[i];
		for (uint32 j = HashTable.First(Hash); HashTable.IsValid(j); j = HashTable.Next(j))
		{
			SimpEdgeType* e1 = &edges[i];
			SimpEdgeType* e2 = &edges[j];

			if (e1 == e2)
				continue;

			bool m1 =
				(e1->v0 == e2->v0 &&
					e1->v1 == e2->v1)
				||
				(e1->v0->GetPos() == e2->v0->GetPos() &&
					e1->v1->GetPos() == e2->v1->GetPos());

			bool m2 =
				(e1->v0 == e2->v1 &&
					e1->v1 == e2->v0)
				||
				(e1->v0->GetPos() == e2->v1->GetPos() &&
					e1->v1->GetPos() == e2->v0->GetPos());

			// backwards
			if (m2)
			{
				Swap(e2->v0, e2->v1);
			}

			// link
			if (m1 || m2)
			{
				checkSlow(e2->next == e2);
				checkSlow(e2->prev == e2);

				e2->next = e1->next;
				e2->prev = e1;
				e2->next->prev = e2;
				e2->prev->next = e2;
			}
		}
	}
}

void ProxyLOD::FQuadricMeshSimplifier::InitCosts()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::FQuadricMeshSimplifier::InitCosts)

	//	SCOPE_LOG_TIME(TEXT("UE_ProxyLOD_Simplifier_InitCosts"), nullptr);
	for (int i = 0; i < edges.Num(); i++)
	{
		float cost = ComputeEdgeCollapseCost(&edges[i]);
		checkSlow(FMath::IsFinite(cost));
		edgeHeap.Add(cost, i);
	}
}


void ProxyLOD::FQuadricMeshSimplifier::RemoveEdge(SimpEdgeType* edge)
{
	if (edge->TestFlags(SIMP_REMOVED))
	{
		checkSlow(edge->next == edge);
		checkSlow(edge->prev == edge);
		return;
	}

	uint32 hash = HashEdge(edge->v0, edge->v1);
	for (uint32 i = edgeHash.First(hash); edgeHash.IsValid(i); i = edgeHash.Next(i))
	{
		if (&edges[i] == edge)
		{
			edgeHash.Remove(hash, i);
			edgeHeap.Remove(i);
			break;
		}
	}

	// remove edge
	edge->EnableFlags(SIMP_REMOVED);

	// ungroup edge
	edge->prev->next = edge->next;
	edge->next->prev = edge->prev;
	edge->next = edge;
	edge->prev = edge;
}

void ProxyLOD::FQuadricMeshSimplifier::ReplaceEdgeVert(const SimpVertType* oldV, const SimpVertType* otherV, SimpVertType* newV)
{
	uint32 hash = HashEdge(oldV, otherV);
	uint32 index;
	for (index = edgeHash.First(hash); edgeHash.IsValid(index); index = edgeHash.Next(index))
	{
		if ((edges[index].v0 == oldV && edges[index].v1 == otherV) ||
			(edges[index].v1 == oldV && edges[index].v0 == otherV))
			break;
	}

	checkSlow(index != -1);
	SimpEdgeType* edge = &edges[index];

	edgeHash.Remove(hash, index);

	SimpEdgeType* ExistingEdge = FindEdge(newV, otherV);
	if (ExistingEdge)
	{
		// Not entirely sure why this happens. I believe these are invalid edges from bridge tris.
		RemoveEdge(ExistingEdge);
	}

	if (newV)
	{
		edgeHash.Add(HashEdge(newV, otherV), index);

		if (edge->v0 == oldV)
			edge->v0 = newV;
		else
			edge->v1 = newV;
	}
	else
	{
		// remove edge
		edge->EnableFlags(SIMP_REMOVED);

		// ungroup old edge
		edge->prev->next = edge->next;
		edge->next->prev = edge->prev;
		edge->next = edge;
		edge->prev = edge;

		edgeHeap.Remove(index);
	}
}


void ProxyLOD::FQuadricMeshSimplifier::CollapseEdgeVert(const SimpVertType* oldV, const SimpVertType* otherV, SimpVertType* newV)
{
	uint32 hash = HashEdge(oldV, otherV);
	uint32 index;
	for (index = edgeHash.First(hash); edgeHash.IsValid(index); index = edgeHash.Next(index))
	{
		if ((edges[index].v0 == oldV && edges[index].v1 == otherV) ||
			(edges[index].v1 == oldV && edges[index].v0 == otherV))
			break;
	}

	checkSlow(index != -1);
	SimpEdgeType* edge = &edges[index];

	edgeHash.Remove(hash, index);
	edgeHeap.Remove(index);

	// remove edge
	edge->EnableFlags(SIMP_REMOVED);

	// ungroup old edge
	edge->prev->next = edge->next;
	edge->next->prev = edge->prev;
	edge->next = edge;
	edge->prev = edge;
}


void ProxyLOD::FQuadricMeshSimplifier::GatherUpdates(SimpVertType* v)
{
	// Update all tris touching collapse edge.
	for (TriIterator i = v->adjTris.Begin(); i != v->adjTris.End(); ++i)
	{
		updateTris.AddUnique(*i);
	}

	TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;
	v->FindAdjacentVerts(adjVerts);

	LockVertFlags(SIMP_MARK1 | SIMP_MARK2);

	// Update verts from tris adjacent to collapsed edge
	for (int i = 0, Num = adjVerts.Num(); i < Num; i++)
	{
		updateVerts.AddUnique(adjVerts[i]);

		adjVerts[i]->EnableFlags(SIMP_MARK2);
	}

	// update the costs of all edges connected to any face adjacent to v
	for (int i = 0, Num = adjVerts.Num(); i < Num; i++)
	{
		adjVerts[i]->EnableAdjVertFlags(SIMP_MARK1);

		for (TriIterator j = adjVerts[i]->adjTris.Begin(); j != adjVerts[i]->adjTris.End(); ++j)
		{
			SimpTriType* tri = *j;
			for (int k = 0; k < 3; k++)
			{
				SimpVertType* vert = tri->verts[k];
				if (vert->TestFlags(SIMP_MARK1) && !vert->TestFlags(SIMP_MARK2))
				{
					SimpEdgeType* edge = FindEdge(adjVerts[i], vert);
					updateEdges.AddUnique(edge);
				}
				vert->DisableFlags(SIMP_MARK1);
			}
		}
		adjVerts[i]->DisableFlags(SIMP_MARK2);
	}

	UnlockVertFlags(SIMP_MARK1 | SIMP_MARK2);
}


template <typename VertexType>
struct OptimizerTrait
{
	typedef ProxyLOD::TQuadricAttrOptimizer<ProxyLOD::FQuadricMeshSimplifier::NumAttributes >  OptimizerType;
};


FVector ProxyLOD::FQuadricMeshSimplifier::ComputeNewVertsPos(SimpEdgeType* edge, TArray< MeshVertType, TInlineAllocator<16> >& newVerts, TArray< QuadricType, TInlineAllocator<16> >& quadrics, FQuadric& edgeQuadric)
{
	SimpEdgeType* e;
	SimpVertType* v;


	typename OptimizerTrait<MeshVertType>::OptimizerType  optimizer;
	//TQuadricAttrOptimizer<TQuadricMeshSimplifier<T>::NumAttributes > optimizer;

	LockVertFlags(SIMP_MARK1);

	edge->v0->EnableFlagsGroup(SIMP_MARK1);
	edge->v1->EnableFlagsGroup(SIMP_MARK1);

	// add edges
	e = edge;
	do {
		checkSlow(e == FindEdge(e->v0, e->v1));
		checkSlow(e->v0->adjTris.Num() > 0);
		checkSlow(e->v1->adjTris.Num() > 0);

		newVerts.Add(e->v0->vert);

		QuadricType quadric;
		quadric = GetQuadric(e->v0);
		quadric += GetQuadric(e->v1);
		quadrics.Add(quadric);
		optimizer.AddQuadric(quadric);

		e->v0->DisableFlags(SIMP_MARK1);
		e->v1->DisableFlags(SIMP_MARK1);

		e = e->next;
	} while (e != edge);

	// add remainder verts
	v = edge->v0;
	do {
		if (v->TestFlags(SIMP_MARK1))
		{
			newVerts.Add(v->vert);

			QuadricType quadric;
			quadric = GetQuadric(v);
			quadrics.Add(quadric);
			optimizer.AddQuadric(quadric);

			v->DisableFlags(SIMP_MARK1);
		}
		v = v->next;
	} while (v != edge->v0);

	v = edge->v1;
	do {
		if (v->TestFlags(SIMP_MARK1))
		{
			newVerts.Add(v->vert);

			QuadricType quadric;
			quadric = GetQuadric(v);
			quadrics.Add(quadric);
			optimizer.AddQuadric(quadric);

			v->DisableFlags(SIMP_MARK1);
		}
		v = v->next;
	} while (v != edge->v1);

	UnlockVertFlags(SIMP_MARK1);

	check(quadrics.Num() <= 256);

	edgeQuadric.Zero();

	v = edge->v0;
	do {
		edgeQuadric += GetEdgeQuadric(v);
		v = v->next;
	} while (v != edge->v0);

	v = edge->v1;
	do {
		edgeQuadric += GetEdgeQuadric(v);
		v = v->next;
	} while (v != edge->v1);

	optimizer.AddQuadric(edgeQuadric);

	FVector newPos;
	{
		bool bLocked0 = edge->v0->TestFlags(SIMP_LOCKED);
		bool bLocked1 = edge->v1->TestFlags(SIMP_LOCKED);
		//checkSlow( !bLocked0 || !bLocked1 ); // can't have both locked

		// find position
		if (bLocked0)
		{
			// v0 position
			newPos = (FVector)edge->v0->GetPos();
		}
		else if (bLocked1)
		{
			// v1 position
			newPos = (FVector)edge->v1->GetPos();
		}
		else
		{
			// optimal position
			bool valid = optimizer.Optimize(newPos);
			if (!valid)
			{
				// Couldn't find optimal so choose middle
				newPos = (FVector)(edge->v0->GetPos() + edge->v1->GetPos()) * 0.5f;
			}
		}
	}

	return newPos;

}


float ProxyLOD::FQuadricMeshSimplifier::ComputeNewVerts(SimpEdgeType* edge, TArray< MeshVertType, TInlineAllocator<16> >& newVerts)
{

	FQuadric edgeQuadric;
	TArray< QuadricType, TInlineAllocator<16> > quadrics;
	FVector newPos = ComputeNewVertsPos(edge, newVerts, quadrics, edgeQuadric);


	float cost = 0.0f;

	for (int i = 0; i < quadrics.Num(); i++)
	{
		newVerts[i].GetPos() = (FVector3f)newPos;

		if (quadrics[i].a > 1e-8)
		{
			// calculate vert attributes from the new position
			quadrics[i].CalcAttributes((FVector)newVerts[i].GetPos(), newVerts[i].GetAttributes(), attributeWeights);
			newVerts[i].Correct();
		}

		// sum cost of new verts
		cost += quadrics[i].Evaluate((FVector)newVerts[i].GetPos(), newVerts[i].GetAttributes(), attributeWeights);
	}

	cost += edgeQuadric.Evaluate(newPos);

	return cost;
}



float ProxyLOD::FQuadricMeshSimplifier::ComputeEdgeCollapseCost(SimpEdgeType* edge)
{
	TArray< MeshVertType, TInlineAllocator<16> > newVerts;

#if 1 
	// djh

	if (edge->v0->TestFlags(SIMP_LOCKED) && edge->v1->TestFlags(SIMP_LOCKED))
	{
		return FLT_MAX;
	}
#endif
	float cost = ComputeNewVerts(edge, newVerts);

	const FVector& newPos = (FVector)newVerts[0].GetPos();

	// add penalties
	// the below penalty code works with groups so no need to worry about remainder verts
	SimpVertType* u = edge->v0;
	SimpVertType* v = edge->v1;
	SimpVertType* vert;

	float penalty = 0.0f;

	{
		//const int degreeLimit = 24;
		//const float degreePenalty = 100.0f;

		int degree = 0;

		// u
		vert = u;
		do {
			degree += vert->adjTris.Num();
			vert = vert->next;
		} while (vert != u);

		// v
		vert = v;
		do {
			degree += vert->adjTris.Num();
			vert = vert->next;
		} while (vert != v);

		if (degree > degreeLimit)
			penalty += degreePenalty * (degree - degreeLimit);
	}


	{

		const float penaltyToPreventEdgeFolding = invalidPenalty;

		LockTriFlags(SIMP_MARK1);

		v->EnableAdjTriFlagsGroup(SIMP_MARK1);


		// u
		vert = u;
		do {
			for (TriIterator i = vert->adjTris.Begin(); i != vert->adjTris.End(); ++i)
			{
				SimpTriType* tri = *i;
				if (!tri->TestFlags(SIMP_MARK1))
				{
					// djh
					if (!tri->ReplaceVertexIsValid(vert, (FVector3f)newPos))
					{
						penalty += penaltyToPreventEdgeFolding;

					}
					//penalty += tri->ReplaceVertexIsValid(vert, newPos, minDotProduct) ? 0.0f : penaltyToPreventEdgeFolding;
				}
				tri->DisableFlags(SIMP_MARK1);
			}
			vert = vert->next;
		} while (vert != u);

		// v
		vert = v;
		do {
			for (TriIterator i = vert->adjTris.Begin(); i != vert->adjTris.End(); ++i)
			{
				SimpTriType* tri = *i;
				if (tri->TestFlags(SIMP_MARK1))
				{
					if (!tri->ReplaceVertexIsValid(vert, (FVector3f)newPos))
					{
						penalty += penaltyToPreventEdgeFolding;

					}
					//penalty += tri->ReplaceVertexIsValid(vert, newPos, minDotProduct) ? 0.0f : penaltyToPreventEdgeFolding;
				}
				tri->DisableFlags(SIMP_MARK1);
			}
			vert = vert->next;
		} while (vert != v);

		UnlockTriFlags(SIMP_MARK1);
	}

	return  cost + penalty;
}

void ProxyLOD::FQuadricMeshSimplifier::Collapse(SimpEdgeType* edge)
{
	SimpVertType* u = edge->v0;
	SimpVertType* v = edge->v1;

	// Collapse the edge uv by moving vertex u onto v
	checkSlow(u && v);
	checkSlow(edge == FindEdge(u, v));
	checkSlow(u->adjTris.Num() > 0);
	checkSlow(v->adjTris.Num() > 0);
	checkSlow(!u->TestFlags(SIMP_LOCKED) || !v->TestFlags(SIMP_LOCKED));


	if (u->TestFlags(SIMP_LOCKED))
		v->EnableFlags(SIMP_LOCKED);

	LockVertFlags(SIMP_MARK1);

	// update edges from u to v
	u->EnableAdjVertFlags(SIMP_MARK1);
	v->DisableAdjVertFlags(SIMP_MARK1);

	if (u->TestFlags(SIMP_MARK1))
	{
		// Invalid edge, results from collapsing a bridge tri
		// There are no actual triangles connecting these verts
		u->DisableAdjVertFlags(SIMP_MARK1);
		UnlockVertFlags(SIMP_MARK1);
		return;
	}

	for (TriIterator i = u->adjTris.Begin(); i != u->adjTris.End(); ++i)
	{
		SimpTriType* tri = *i;
		for (int j = 0; j < 3; j++)
		{
			SimpVertType* vert = tri->verts[j];
			if (vert->TestFlags(SIMP_MARK1))
			{
				ReplaceEdgeVert(u, vert, v);
				vert->DisableFlags(SIMP_MARK1);
			}
		}
	}

	// remove dead edges
	u->EnableAdjVertFlags(SIMP_MARK1);
	u->DisableFlags(SIMP_MARK1);
	v->DisableFlags(SIMP_MARK1);

	for (TriIterator i = v->adjTris.Begin(); i != v->adjTris.End(); ++i)
	{
		SimpTriType* tri = *i;
		for (int j = 0; j < 3; j++)
		{
			SimpVertType* vert = tri->verts[j];
			if (vert->TestFlags(SIMP_MARK1))
			{
				ReplaceEdgeVert(u, vert, NULL);
				vert->DisableFlags(SIMP_MARK1);
			}
		}
	}

	u->DisableAdjVertFlags(SIMP_MARK1);

	// fixup triangles
	for (TriIterator i = u->adjTris.Begin(); i != u->adjTris.End(); ++i)
	{
		SimpTriType* tri = *i;
		checkSlow(!tri->TestFlags(SIMP_REMOVED));
		checkSlow(tri->HasVertex(u));

		if (tri->HasVertex(v))
		{
			// delete triangles on edge uv
			numTris--;
			tri->EnableFlags(SIMP_REMOVED);
			quadricCache.DirtyTriQuadric(tri);

			// remove references to tri
			for (int j = 0; j < 3; j++)
			{
				SimpVertType* vert = tri->verts[j];
				checkSlow(!vert->TestFlags(SIMP_REMOVED));
				if (vert != u)
				{
					vert->adjTris.Remove(tri);
				}
			}
		}
		else
		{
			// update triangles to have v instead of u
			tri->ReplaceVertex(u, v);
			v->adjTris.Add(tri);
		}
	}

	// remove modified verts and tris from cache
	v->EnableAdjVertFlags(SIMP_MARK1);
	for (TriIterator i = v->adjTris.Begin(); i != v->adjTris.End(); ++i)
	{
		SimpTriType* tri = *i;
		quadricCache.DirtyTriQuadric(tri);

		for (int j = 0; j < 3; j++)
		{
			SimpVertType* vert = tri->verts[j];
			if (vert->TestFlags(SIMP_MARK1))
			{
				quadricCache.DirtyVertQuadric(vert);
				vert->DisableFlags(SIMP_MARK1);
			}
		}
	}


	UnlockVertFlags(SIMP_MARK1);

	u->adjTris.Clear();	// u has been removed
	u->EnableFlags(SIMP_REMOVED);
	numVerts--;
}


void ProxyLOD::FQuadricMeshSimplifier::UpdateTris()
{
	// remove degenerate triangles
	// not sure why this happens
	for (SimpTriType* tri : updateTris)
	{
		if (tri->TestFlags(SIMP_REMOVED))
			continue;

		quadricCache.DirtyTriQuadric(tri);

		const FVector& p0 = (FVector)tri->verts[0]->GetPos();
		const FVector& p1 = (FVector)tri->verts[1]->GetPos();
		const FVector& p2 = (FVector)tri->verts[2]->GetPos();
		const FVector n = (p2 - p0) ^ (p1 - p0);

		if (n.SizeSquared() == 0.0f)
		{
			numTris--;
			tri->EnableFlags(SIMP_REMOVED);

			// remove references to tri
			for (int j = 0; j < 3; j++)
			{
				SimpVertType* vert = tri->verts[j];
				vert->adjTris.Remove(tri);
				// orphaned verts are removed below
			}
		}
	}
	updateTris.Reset();
}


void ProxyLOD::FQuadricMeshSimplifier::UpdateVerts()
{
	// remove orphaned verts
	for (SimpVertType* vert : updateVerts)
	{
		if (vert->TestFlags(SIMP_REMOVED))
			continue;

		quadricCache.DirtyVertQuadric(vert);
		quadricCache.DirtEdgeQuadric(vert);


		if (vert->adjTris.Num() == 0)
		{
			numVerts--;
			vert->EnableFlags(SIMP_REMOVED);

			// ungroup
			vert->prev->next = vert->next;
			vert->next->prev = vert->prev;
			vert->next = vert;
			vert->prev = vert;
		}
	}
	updateVerts.Reset();
}


void ProxyLOD::FQuadricMeshSimplifier::UpdateEdges()
{
	const uint32 NumEdges = updateEdges.Num();

	// add all grouped edges
	for (uint32 i = 0; i < NumEdges; i++)
	{
		SimpEdgeType* edge = updateEdges[i];

		if (edge->TestFlags(SIMP_REMOVED))
			continue;

		SimpEdgeType* e = edge;
		do {
			updateEdges.AddUnique(e);
			e = e->next;
		} while (e != edge);
	}

	// remove dead edges from our edge hash.
	for (uint32 i = 0, Num = updateEdges.Num(); i < Num; i++)
	{
		SimpEdgeType* edge = updateEdges[i];

		if (edge->TestFlags(SIMP_REMOVED))
			continue;

		if (edge->v0->TestFlags(SIMP_REMOVED) ||
			edge->v1->TestFlags(SIMP_REMOVED))
		{
			RemoveEdge(edge);
		}
	}

	// Fix edge groups
	{
		FHashTable HashTable(128, NumEdges);

		// ungroup edges
		for (uint32 i = 0, Num = updateEdges.Num(); i < Num; i++)
		{
			SimpEdgeType* edge = updateEdges[i];

			if (edge->TestFlags(SIMP_REMOVED))
				continue;

			edge->next = edge;
			edge->prev = edge;

			HashTable.Add(HashPoint((FVector)edge->v0->GetPos()) ^ HashPoint((FVector)edge->v1->GetPos()), i);
		}

		// regroup edges
		for (uint32 i = 0, Num = updateEdges.Num(); i < Num; i++)
		{
			SimpEdgeType* edge = updateEdges[i];

			if (edge->TestFlags(SIMP_REMOVED))
				continue;

			// already grouped
			if (edge->next != edge)
				continue;

			// find any matching edges
			uint32 hash = HashPoint((FVector)edge->v0->GetPos()) ^ HashPoint((FVector)edge->v1->GetPos());
			SimpEdgeType* e1 = updateEdges[i];
			for (uint32 j = HashTable.First(hash); HashTable.IsValid(j); j = HashTable.Next(j))
			{

				SimpEdgeType* e2 = updateEdges[j];

				if (e1 == e2)
					continue;

				bool m1 =
					(e1->v0 == e2->v0 &&
						e1->v1 == e2->v1)
					||
					(e1->v0->GetPos() == e2->v0->GetPos() &&
						e1->v1->GetPos() == e2->v1->GetPos());

				bool m2 =
					(e1->v0 == e2->v1 &&
						e1->v1 == e2->v0)
					||
					(e1->v0->GetPos() == e2->v1->GetPos() &&
						e1->v1->GetPos() == e2->v0->GetPos());

				// backwards
				if (m2)
					Swap(e2->v0, e2->v1);

				// link
				if (m1 || m2)
				{
					checkSlow(e2->next == e2);
					checkSlow(e2->prev == e2);

					e2->next = e1->next;
					e2->prev = e1;
					e2->next->prev = e2;
					e2->prev->next = e2;
				}
			}
		}
	}

	// update edges
	for (uint32 i = 0; i < NumEdges; i++)
	{
		SimpEdgeType* edge = updateEdges[i];

		if (edge->TestFlags(SIMP_REMOVED))
			continue;

		float cost = ComputeEdgeCollapseCost(edge);

		SimpEdgeType* e = edge;
		do {
			uint32 EdgeIndex = GetEdgeIndex(e);
			if (edgeHeap.IsPresent(EdgeIndex))
			{
				edgeHeap.Update(cost, EdgeIndex);
			}
			e = e->next;
		} while (e != edge);
	}
	updateEdges.Reset();
}

int32 ProxyLOD::FQuadricMeshSimplifier::CountDegenerates() const
{
	int32 DegenerateCount = 0;
	// remove degenerate triangles
	// not sure why this happens
	for (int i = 0; i < numSTris; i++)
	{
		SimpTriType* tri = &sTris[i];

		if (tri->TestFlags(SIMP_REMOVED))
			continue;

		const FVector& p0 = (FVector)tri->verts[0]->GetPos();
		const FVector& p1 = (FVector)tri->verts[1]->GetPos();
		const FVector& p2 = (FVector)tri->verts[2]->GetPos();
		const FVector n = (p2 - p0) ^ (p1 - p0);

		if (n.SizeSquared() == 0.0f)
		{
			DegenerateCount++;
		}
	}

	return DegenerateCount;
}


void ProxyLOD::FQuadricMeshSimplifier::OutputMesh(MeshVertType* verts, uint32* indexes, TArray<int32>* LockedVerts, int* OutNumVertices, int* OutNumIndices) const
{
	FHashTable HashTable(4096, GetNumVerts());

#if 1
	int count = 0;
	for (int i = 0; i < numSVerts; i++)
		count += sVerts[i].TestFlags(SIMP_REMOVED) ? 0 : 1;

	check(numVerts == count);
#endif

	int numV = 0;
	int numI = 0;

	for (int i = 0; i < numSTris; i++)
	{
		if (sTris[i].TestFlags(SIMP_REMOVED))
			continue;

		// TODO this is sloppy. There should be no duped verts. Alias by index.
		for (int j = 0; j < 3; j++)
		{
			SimpVertType* vert = sTris[i].verts[j];
			checkSlow(!vert->TestFlags(SIMP_REMOVED));
			checkSlow(vert->adjTris.Num() != 0);

			const FVector& p = (FVector)vert->GetPos();
			uint32 hash = HashPoint(p);
			uint32 f;
			for (f = HashTable.First(hash); HashTable.IsValid(f); f = HashTable.Next(f))
			{
				if (vert->vert == verts[f])
					break;
			}
			if (!HashTable.IsValid(f))
			{
				// export the id of the locked vert.
				if (LockedVerts != NULL && vert->TestFlags(SIMP_LOCKED))
				{
					LockedVerts->Push(numV);
				}

				HashTable.Add(hash, numV);
				verts[numV] = vert->vert;
				indexes[numI++] = numV;
				numV++;
			}
			else
			{
				indexes[numI++] = f;
			}
		}
	}

	check(numV <= numVerts);
	check(numI <= numTris * 3);

	if (OutNumVertices)
	{
		*OutNumVertices = numV;
	}

	if (OutNumIndices)
	{
		*OutNumIndices = numI;
	}
}