#pragma once
// Copyright (C) 2009 Nine Realms, Inc
//

#include "CoreMinimal.h"


#include "ProxyLODMeshTypes.h" // for resize array
#include "ProxyLODThreadedWrappers.h"

#include "Containers/BinaryHeap.h"
#include "Containers/HashTable.h"
#include "MeshSimplifyElements.h"
#include "ProxyLODQuadric.h"


//===========================================================================================

namespace ProxyLOD
{

	/**
	*  Define the Simplifier types needed.
	*/
	typedef FPositionNormalVertex      MeshVertType;
	typedef TSimpVert<MeshVertType>    SimpVertType;
	typedef TSimpTri<MeshVertType>     SimpTriType;
	typedef TSimpEdge<MeshVertType>    SimpEdgeType;

	/**
	* Cache to manage quadrics for the quadric mesh reduction.  
	* 
	*/
	template <typename QuadricType>
	class TQuadricCache
	{
	public:

		TQuadricCache(int32 NumSVerts, int32 NumSTris);

		/**
		* Associate the cache with the simplifier vert and triangle arrays.
		* This must be done before the cache can be used.
		*
		* @param VertOffset  Pointer to the array of verts used by the simplifier
		* @param TriOffset   Pointer to the array of Tris  used by the simplifier.
		*/
		void RegisterCache(const SimpVertType* VertOffset, const SimpTriType* TriOffset);

		/**
		* Get the Quadric.
		* If the quadric for the requested SimplfierVert is out of date (dirty) then use
		* the supplied TriQuadricFactor functor to compute a new quadric (and add it to the cache)
		*
		* QuadricType  TriQuadricFactor(SimpTriType& Tri);
		*
		* @param v                  The vert
		* @param TriQuadricFactor   Method to compute new quadric from a triangle
		*/
		template <typename TriQuadricFatoryType>
		QuadricType GetQuadric(SimpVertType* v, const TriQuadricFatoryType& TriQuadricFactory);


		/**
		* Get the Edge Quadric.
		* If the edge quadric for the requested SimplfierVert is out of date (dirty) then use
		* the supplied EdgeQuadricFatory functor to compute a new quadric (and add it to the cache)
		*
		* FQuadric  EdgeQuadricFatory(v->GetPos(), vert->GetPos(), face->GetNormal());
		*
		* @param v                  The vert
		* @param EdgeQuadricFatory   Method to compute new quadric from a triangle
		*/
		template <typename EdgeQuadricFatoryType>
		ProxyLOD::FQuadric GetEdgeQuadric(SimpVertType* v, const EdgeQuadricFatoryType& EdgeQuadricFatory);


		/**
		* Mark the associated vertex quadric as dirty in the cache.
		*/
		void DirtyVertQuadric(const SimpVertType* v);
		
		/**
		* Mark the associated triangle quadric as dirty in the cache.
		*/
		void DirtyTriQuadric(const SimpTriType* tri);
	
		/**
		* Mark the associated edge quadric as dirty in the cache.
		*/
		void DirtEdgeQuadric(const SimpVertType* v);
		
		
	private:
		uint32 GetVertIndex(const SimpVertType* vert) const;
		
		uint32 GetTriIndex(const SimpTriType* tri) const;
		

	private:
		// Disable

		TQuadricCache();

	private:

		TBitArray<>				VertQuadricsValid;
		TArray< QuadricType >	VertQuadrics;

		TBitArray<>				TriQuadricsValid;
		TArray< QuadricType >	TriQuadrics;

		TBitArray<>				EdgeQuadricsValid;
		TArray< ProxyLOD::FQuadric >		EdgeQuadrics;

		// To map vert pointer to vert index

		const SimpVertType*		sVerts = NULL;
		const SimpTriType*		sTris = NULL;
	};




	

	template <typename VertType>
	struct VertToQuadricTrait
	{
		typedef TQuadricAttr< VertType::NumAttributesValue > QuadricType;
	};
	template<> struct VertToQuadricTrait<FPositionOnlyVertex>
	{
		typedef ProxyLOD::FQuadric  QuadricType;
	};

	/**
	* Simple terminator class.  This should be expanded on to include an interrupter to allow the user
	* to halt execution. 
	*/
	class FSimplifierTerminatorBase
	{
	public:
		FSimplifierTerminatorBase(int32 MinTri, float MaxCost)
			: MaxFeatureCost(MaxCost), MinTriNumToRetain(MinTri) {}

		// return true if the simplifier should terminate.
		inline bool operator()(const int32 TriNum, const float SqrError)
		{
			if (TriNum < MinTriNumToRetain || SqrError > MaxFeatureCost)
			{
				return true;
			}
			return false;
		}

		float MaxFeatureCost;
		int32 MinTriNumToRetain;
	};

	/**
	* Termination criterion for single threaded simplifier
	*/
	class FSimplifierTerminator : public FSimplifierTerminatorBase
	{
	public:
		FSimplifierTerminator(int32 MinTri, int32 MaxTri, float MaxCost)
			: FSimplifierTerminatorBase(MinTri, MaxCost), MaxTriNumToRetain(MaxTri) {}

		// return true if the simplifier should terminate.
		inline bool operator()(const int32 TriNum, const float SqrError) 
		{
			if (FSimplifierTerminatorBase::operator()(TriNum, SqrError) && TriNum < MaxTriNumToRetain)
			{
				return true;
			}
			else
			{
				return false;
			}
		}
	
		int32 MaxTriNumToRetain;

	};

	/**
	*  Slightly modified version of the quadric simplifier found in MeshSimplifier\Private\MeshSimplify.h
	*  that code caries the copyright --
	*/
	// Copyright (C) 2009 Nine Realms, Inc
	class FQuadricMeshSimplifier
	{
		typedef typename SimpVertType::TriIterator	                         TriIterator;
		typedef typename VertToQuadricTrait<MeshVertType>::QuadricType       QuadricType;
		typedef TQuadricCache<QuadricType>                                   QuadricCacheType;

	public:
		enum { NumAttributes = MeshVertType::NumAttributesValue };
		FQuadricMeshSimplifier(const  MeshVertType* Verts, uint32 NumVerts, const uint32* Indexes, uint32 NumIndexes, float CoAlignmentLimit);
		~FQuadricMeshSimplifier();

		void				SetAttributeWeights(const float* weights);
		void				SetBoundaryLocked();

		void				InitCosts();

							template <typename TerminationCriterionType>
	   float				SimplifyMesh(TerminationCriterionType TerminationCriterion);

		int					GetNumVerts() const { return numVerts; }
		int					GetNumTris() const { return numTris; }

		void				OutputMesh(MeshVertType* Verts, uint32* Indexes, TArray<int32>* LockedVerts = NULL, int* OutNumVerts = NULL, int* OutNumIndexes = NULL) const;

	protected:
		void				LockVertFlags(uint32 flag);
		void				UnlockVertFlags(uint32 flag);

		void				LockTriFlags(uint32 flag);
		void				UnlockTriFlags(uint32 flag);

		void				GatherUpdates(SimpVertType* v);

		void				GroupVerts();
		void				GroupEdges();

		void				InitVert(SimpVertType* v);

		QuadricType			GetQuadric(SimpVertType* v);
		ProxyLOD::FQuadric			GetEdgeQuadric(SimpVertType* v);

		// TODO move away from pointers and remove these functions
		uint32				GetVertIndex(const SimpVertType* vert) const;
		uint32				GetTriIndex(const SimpTriType* tri) const;
		uint32				GetEdgeIndex(const SimpEdgeType* edge) const;

		uint32				HashPoint(const FVector& p) const;
		uint32				HashEdge(const SimpVertType* u, const SimpVertType* v) const;
		SimpEdgeType*		FindEdge(const SimpVertType* u, const SimpVertType* v);

		void				RemoveEdge(SimpEdgeType* edge);
		void				ReplaceEdgeVert(const SimpVertType* oldV, const SimpVertType* otherV, SimpVertType* newV);
		void				CollapseEdgeVert(const SimpVertType* oldV, const SimpVertType* otherV, SimpVertType* newV);

		float				ComputeNewVerts(SimpEdgeType* edge, TArray< MeshVertType, TInlineAllocator<16> >& newVerts);
		FVector             ComputeNewVertsPos(SimpEdgeType* edge, TArray< MeshVertType, TInlineAllocator<16> >& newVerts, TArray< QuadricType, TInlineAllocator<16> >& quadrics, ProxyLOD::FQuadric& edgeQuadric);
		
		float				ComputeEdgeCollapseCost(SimpEdgeType* edge);
		void				Collapse(SimpEdgeType* edge);

		void				UpdateTris();
		void				UpdateVerts();
		void				UpdateEdges();

		
		int32               CountDegenerates() const; // Included for testing.

		uint32				vertFlagLock;
		uint32				triFlagLock;

		float				attributeWeights[NumAttributes+1];// Note the +1 element is never used.

		SimpVertType*		sVerts;
		SimpTriType*		sTris;

		int					numSVerts  = 0;
		int					numSTris   = 0;

		int					numVerts   = 0;
		int					numTris    = 0;

		// --Magic numbers that penalize undesirable simplifications.

		// prevent high valence verts.
		int                 degreeLimit       = 24;
		float               degreePenalty    = 100.0f;

		// prevent edge folding 
		float               invalidPenalty   = 1.e7;
		float               coAlignmentLimit = .0871557f;  // 85-degrees  -- currently not used!
		

		TArray< SimpEdgeType >	edges;
		FHashTable				edgeHash;
		FBinaryHeap<float>		edgeHeap;

		// Manage the quadrics.

		QuadricCacheType        quadricCache;

		TArray< SimpVertType* >	updateVerts;
		TArray< SimpTriType* >	updateTris;
		TArray< SimpEdgeType* >	updateEdges;

	private:
			FQuadricMeshSimplifier();
		
	};


	// locking functions for nesting safety
	FORCEINLINE void FQuadricMeshSimplifier::LockVertFlags(uint32 f)
	{
		checkSlow((vertFlagLock & f) == 0);
		vertFlagLock |= f;
	}

	FORCEINLINE void FQuadricMeshSimplifier::UnlockVertFlags(uint32 f)
	{
		vertFlagLock &= ~f;
	}

	FORCEINLINE void FQuadricMeshSimplifier::LockTriFlags(uint32 f)
	{
		checkSlow((triFlagLock & f) == 0);
		triFlagLock |= f;
	}

	FORCEINLINE void FQuadricMeshSimplifier::UnlockTriFlags(uint32 f)
	{
		triFlagLock &= ~f;
	}


	


	FORCEINLINE  FQuadricMeshSimplifier::QuadricType FQuadricMeshSimplifier::GetQuadric(SimpVertType* v)
	{
		
		const auto 	TriQuadricFatory = [this](const SimpTriType& tri)->QuadricType
		{
			return QuadricType(
				(FVector)tri.verts[0]->GetPos(), (FVector)tri.verts[1]->GetPos(), (FVector)tri.verts[2]->GetPos(),
				tri.verts[0]->GetAttributes(), tri.verts[1]->GetAttributes(), tri.verts[2]->GetAttributes(),
				this->attributeWeights);
		};

		return quadricCache.GetQuadric(v, TriQuadricFatory);
	}

	
	FORCEINLINE ProxyLOD::FQuadric FQuadricMeshSimplifier::GetEdgeQuadric(SimpVertType* v)
	{

		const auto EdgeQuadricFactory = [this](const FVector& Pos0, const FVector& Pos1, const FVector& Normal)->ProxyLOD::FQuadric
		{
			return ProxyLOD::FQuadric(Pos0, Pos1, Normal, 256.f);
		};
		LockTriFlags(SIMP_MARK1);
		ProxyLOD::FQuadric Quadric = quadricCache.GetEdgeQuadric(v, EdgeQuadricFactory);
		UnlockTriFlags(SIMP_MARK1);
		return Quadric;
	}

	FORCEINLINE uint32 FQuadricMeshSimplifier::GetVertIndex(const SimpVertType* vert) const
	{
		ptrdiff_t Index = vert - &sVerts[0];
		return (uint32)Index;
	}

	FORCEINLINE uint32 FQuadricMeshSimplifier::GetTriIndex(const SimpTriType* tri) const
	{
		ptrdiff_t Index = tri - &sTris[0];
		return (uint32)Index;
	}


	FORCEINLINE uint32 FQuadricMeshSimplifier::GetEdgeIndex(const SimpEdgeType* edge) const
	{
		ptrdiff_t Index = edge - &edges[0];
		return (uint32)Index;
	}

	FORCEINLINE uint32 FQuadricMeshSimplifier::HashPoint(const FVector& p) const
	{
		union { float f; uint32 i; } x;
		union { float f; uint32 i; } y;
		union { float f; uint32 i; } z;

		x.f = p.X;
		y.f = p.Y;
		z.f = p.Z;

		return Murmur32({ x.i, y.i, z.i });
	}


	FORCEINLINE uint32 FQuadricMeshSimplifier::HashEdge(const SimpVertType* u, const SimpVertType* v) const
	{
		uint32 ui = GetVertIndex(u);
		uint32 vi = GetVertIndex(v);
		// must be symmetrical
		return Murmur32({ FMath::Min(ui, vi), FMath::Max(ui, vi) });
	}

	
	FORCEINLINE SimpEdgeType* FQuadricMeshSimplifier::FindEdge(const SimpVertType* u, const SimpVertType* v)
	{
		uint32 hash = HashEdge(u, v);
		for (uint32 i = edgeHash.First(hash); edgeHash.IsValid(i); i = edgeHash.Next(i))
		{
			if ((edges[i].v0 == u && edges[i].v1 == v) ||
				(edges[i].v0 == v && edges[i].v1 == u))
			{
				return &edges[i];
			}
		}

		return NULL;
	}


	template< typename TerminationCriterionType>
	float FQuadricMeshSimplifier::SimplifyMesh(TerminationCriterionType TerminationCriterion)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FQuadricMeshSimplifier::SimplifyMesh)

		SimpVertType* v;
		SimpEdgeType* e;

		float maxError = 0.0f;
		// One trace scope per batch is enough for Insights and is easier to read
		const int32 BatchSize = 4096;

		while (edgeHeap.Num() > 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SimplifyMeshBatch)

			int32 Batch = 0;
			for ( ; Batch < BatchSize && edgeHeap.Num() > 0; ++Batch)
			{
				// get the next vertex to collapse
				uint32 TopIndex = edgeHeap.Top();

				const float error = edgeHeap.GetKey(TopIndex);

				if (TerminationCriterion(numTris, error)) break;


				maxError = FMath::Max(maxError, error);

				edgeHeap.Pop();

				SimpEdgeType* top = &edges[TopIndex];
				check(top);

				int numEdges = 0;
				SimpEdgeType* edgeList[32];

				SimpEdgeType* edge = top;
				do {
					edgeList[numEdges++] = edge;
					edge = edge->next;
				} while (edge != top);

			
				// skip locked edges
				bool locked = false;
				for (int i = 0; i < numEdges; i++)
				{
					edge = edgeList[i];
					if (edge->v0->TestFlags(SIMP_LOCKED) && edge->v1->TestFlags(SIMP_LOCKED))
					{
						locked = true;
					}
				}
				if (locked)
				{
					continue;
				}
				

				v = top->v0;
				do {
					GatherUpdates(v);
					v = v->next;
				} while (v != top->v0);

				v = top->v1;
				do {
					GatherUpdates(v);
					v = v->next;
				} while (v != top->v1);

	#if 1
				// remove edges with already removed verts
				// not sure why this happens
				for (int i = 0; i < numEdges; i++)
				{
					if (edgeList[i]->v0->adjTris.Num() == 0 ||
						edgeList[i]->v1->adjTris.Num() == 0)
					{
						RemoveEdge(edgeList[i]);
						edgeList[i] = NULL;
					}
					else
					{
						checkSlow(!edgeList[i]->TestFlags(SIMP_REMOVED));
					}
				}
				if (top->v0->adjTris.Num() == 0 || top->v1->adjTris.Num() == 0)
					continue;
	#endif

				// move verts to new verts
				{
					edge = top;

					TArray< MeshVertType, TInlineAllocator<16> > newVerts;
					ComputeNewVerts(edge, newVerts);

					uint32 i = 0;

					LockVertFlags(SIMP_MARK1);

					edge->v0->EnableFlagsGroup(SIMP_MARK1);
					edge->v1->EnableFlagsGroup(SIMP_MARK1);

					// edges
					e = edge;
					do {
						checkSlow(e == FindEdge(e->v0, e->v1));
						checkSlow(e->v0->adjTris.Num() > 0);
						checkSlow(e->v1->adjTris.Num() > 0);

						e->v1->vert = newVerts[i++];
						e->v0->DisableFlags(SIMP_MARK1);
						e->v1->DisableFlags(SIMP_MARK1);

						e = e->next;
					} while (e != edge);

					// remainder verts
					v = edge->v0;
					do {
						if (v->TestFlags(SIMP_MARK1))
						{
							v->vert = newVerts[i++];
							v->DisableFlags(SIMP_MARK1);
						}
						v = v->next;
					} while (v != edge->v0);

					v = edge->v1;
					do {
						if (v->TestFlags(SIMP_MARK1)) {
							v->vert = newVerts[i++];
							v->DisableFlags(SIMP_MARK1);
						}
						v = v->next;
					} while (v != edge->v1);

					UnlockVertFlags(SIMP_MARK1);
				}

				// collapse all edges
				for (int i = 0; i < numEdges; i++)
				{
					edge = edgeList[i];
					if (!edge)
						continue;
					if (edge->TestFlags(SIMP_REMOVED))	// wtf?
						continue;
					if (edge->v0->adjTris.Num() == 0)
						continue;
					if (edge->v1->adjTris.Num() == 0)
						continue;

					Collapse(edge);
					RemoveEdge(edge);
				}

				// add v0 remainder verts to v1
				{
					// combine v0 and v1 groups
					top->v0->next->prev = top->v1->prev;
					top->v1->prev->next = top->v0->next;
					top->v0->next = top->v1;
					top->v1->prev = top->v0;

					// ungroup removed verts
					uint32	vertListNum = 0;
					SimpVertType*	vertList[256];

					v = top->v1;
					do {
						vertList[vertListNum++] = v;
						v = v->next;
					} while (v != top->v1);

					check(vertListNum <= 256);

					for (uint32 i = 0; i < vertListNum; i++)
					{
						v = vertList[i];
						if (v->TestFlags(SIMP_REMOVED))
						{
							// ungroup
							v->prev->next = v->next;
							v->next->prev = v->prev;
							v->next = v;
							v->prev = v;
						}
					}
				}

				{
					// spread locked flag to vert group
					uint32 flags = 0;

					v = top->v1;
					do {
						flags |= v->flags & SIMP_LOCKED;
						v = v->next;
					} while (v != top->v1);

					v = top->v1;
					do {
						v->flags |= flags;
						v = v->next;
					} while (v != top->v1);
				}

				UpdateTris();
				UpdateVerts();
				UpdateEdges();
			}

			if (Batch < BatchSize)
			{
				break;
			}
		}
		// remove degenerate triangles
		// not sure why this happens
		for (int i = 0; i < numSTris; i++)
		{
			SimpTriType* tri = &sTris[i];

			if (tri->TestFlags(SIMP_REMOVED))
				continue;

			const FVector& p0 = (FVector)tri->verts[0]->GetPos();
			const FVector& p1 = (FVector)tri->verts[1]->GetPos();
			const FVector& p2 = (FVector)tri->verts[2]->GetPos();
			const FVector n = (p2 - p0) ^ (p1 - p0);

			if (n.SizeSquared() == 0.0f)
			{
				numTris--;
				tri->EnableFlags(SIMP_REMOVED);

				// remove references to tri
				for (int j = 0; j < 3; j++)
				{
					SimpVertType* vert = tri->verts[j];
					vert->adjTris.Remove(tri);
					// orphaned verts are removed below
				}
			}
		}

		// remove orphaned verts
		for (int i = 0; i < numSVerts; i++)
		{
			SimpVertType* vert = &sVerts[i];

			if (vert->TestFlags(SIMP_REMOVED))
				continue;

			if (vert->adjTris.Num() == 0)
			{
				numVerts--;
				vert->EnableFlags(SIMP_REMOVED);
			}
		}

		return maxError;
	}


	
	//=============
	// TQuadricCache
	//=============

	template <typename QuadricType >
	TQuadricCache< QuadricType >::TQuadricCache(int32 NumSVerts, int32 NumSTris)
	{
		VertQuadricsValid.Init(false, NumSVerts);
		VertQuadrics.SetNum(NumSVerts);

		TriQuadricsValid.Init(false, NumSTris);
		TriQuadrics.SetNum(NumSTris);

		EdgeQuadricsValid.Init(false, NumSVerts);
		EdgeQuadrics.SetNum(NumSVerts);
	}


	template <typename QuadricType>
	void TQuadricCache< QuadricType>::RegisterCache(const SimpVertType* VertOffset, const SimpTriType* TriOffset)
	{
		sVerts = VertOffset;
		sTris = TriOffset;
	}

	// Get the quadric 


	template <typename QuadricType>
	template <typename TriQuadricFatoryType>
	QuadricType TQuadricCache< QuadricType>::GetQuadric(SimpVertType* v, const TriQuadricFatoryType& TriQuadricFactory)
	{

		uint32 VertIndex = GetVertIndex(v);
		if (VertQuadricsValid[VertIndex])
		{
			return VertQuadrics[VertIndex];
		}

		QuadricType vertQuadric;
		vertQuadric.Zero();

		// sum tri quadrics
		for (auto i = v->adjTris.Begin(); i != v->adjTris.End(); ++i)
		{
			SimpTriType* tri = *i;
			uint32 TriIndex = GetTriIndex(tri);
			if (TriQuadricsValid[TriIndex])
			{
				vertQuadric += TriQuadrics[TriIndex];
			}
			else
			{
				QuadricType triQuadric = TriQuadricFactory(*tri);
				vertQuadric += triQuadric;

				TriQuadricsValid[TriIndex] = true;
				TriQuadrics[TriIndex] = triQuadric;
			}
		}

		VertQuadricsValid[VertIndex] = true;
		VertQuadrics[VertIndex] = vertQuadric;

		return vertQuadric;
	}



	template <typename QuadricType>
	template <typename EdgeQuadricFatoryType>
	ProxyLOD::FQuadric TQuadricCache< QuadricType>::GetEdgeQuadric(SimpVertType* v, const EdgeQuadricFatoryType& EdgeQuadricFatory)
	{

		uint32 VertIndex = GetVertIndex(v);
		if (EdgeQuadricsValid[VertIndex])
		{
			return EdgeQuadrics[VertIndex];
		}

		ProxyLOD::FQuadric vertQuadric;
		vertQuadric.Zero();

		TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;
		v->FindAdjacentVerts(adjVerts);

		// djh 
		//	LockTriFlags(SIMP_MARK1);
		v->EnableAdjTriFlags(SIMP_MARK1);

		for (SimpVertType* vert : adjVerts)
		{
			SimpTriType* face = NULL;
			int faceCount = 0;
			for (auto j = vert->adjTris.Begin(); j != vert->adjTris.End(); ++j)
			{
				SimpTriType* tri = *j;
				if (tri->TestFlags(SIMP_MARK1))
				{
					face = tri;
					faceCount++;
				}
			}

			if (faceCount == 1)
			{
				// only one face on this edge
				vertQuadric += EdgeQuadricFatory((FVector)v->GetPos(), (FVector)vert->GetPos(), (FVector)face->GetNormal());
			}
		}


		v->DisableAdjTriFlags(SIMP_MARK1);
		//	UnlockTriFlags(SIMP_MARK1);

		EdgeQuadricsValid[VertIndex] = true;
		EdgeQuadrics[VertIndex] = vertQuadric;

		return vertQuadric;
	}
	

	template <typename QuadricType>
	FORCEINLINE void TQuadricCache< QuadricType>::DirtyVertQuadric(const SimpVertType* v)
	{
		VertQuadricsValid[GetVertIndex(v)] = false;
	}

	template <typename QuadricType>
	FORCEINLINE void TQuadricCache< QuadricType>::DirtyTriQuadric(const SimpTriType* tri)
	{
		TriQuadricsValid[GetTriIndex(tri)] = false;
	}

	template <typename QuadricType>
	FORCEINLINE void TQuadricCache< QuadricType>::DirtEdgeQuadric(const SimpVertType* v)
	{
		EdgeQuadricsValid[GetVertIndex(v)] = false;
	}

	template <typename QuadricType>
	FORCEINLINE uint32 TQuadricCache< QuadricType>::GetVertIndex(const SimpVertType* vert) const
	{
		ptrdiff_t Index = vert - &sVerts[0];
		return (uint32)Index;
	}

	template <typename QuadricType>
	FORCEINLINE uint32 TQuadricCache< QuadricType>::GetTriIndex(const SimpTriType* tri) const
	{
		ptrdiff_t Index = tri - &sTris[0];
		return (uint32)Index;
	}

}; // end namespace ProxyLOD
