// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

THIRD_PARTY_INCLUDES_START
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#include <tbb/blocked_range.h>
#include <tbb/blocked_range2d.h>
#include <tbb/concurrent_vector.h>
#include <tbb/parallel_for.h>
#include <tbb/parallel_reduce.h>
#include <tbb/task_group.h>
#include <tbb/task_scheduler_observer.h>
PRAGMA_ENABLE_DEPRECATION_WARNINGS
THIRD_PARTY_INCLUDES_END

/**
* Wrappers for the tbb calls to allow for easier single threaded testing.
*
* The ProxyLOD system depends on a thirdparty library (openvdb) that requires 
* Intel's threaded building blocks (tbb).  
*
* To maintain the parallelism model, the ProxyLOD system uses tbb for all internal threading.  
* This allows us to exploit the inherent load balancing provided by this compossible task-based
* threading model, and share the underlying task-manager with the third-party code. 
*
* NB: These methods are fully compossible.  Meaning that nesting of parallel calls 
*     (e.g. calling a Parallel_For within a Parallel_Reduce or any task group) is admissible. 
*/

namespace ProxyLOD
{
	/**
	* Splittable Range Types for parallel loops.
	*
	* NB: to satisfy load balancing needs, the parallel loop-based algorithms can split the range of an item
	*     of work to make smaller tasks.
	*/
	typedef tbb::blocked_range<int32>    FIntRange;
	typedef tbb::blocked_range<uint32>   FUIntRange;
	typedef tbb::blocked_range2d<int32>  FIntRange2d;

	/**
	* Range-based parallel for that supports internal range splitting.  The work may be split into multiple tasks.
	* 
	* -------------------------------------------------------------------------------------------
	* Example:  Data transformation of an array.
	*
	* TArray<float>  SrcArray;
	* ... 
	* // target for data transformation
	* TArray<float>  DstArray;
	*
	* // prep for parallel write 
	* Resize(DstArray, SrcArray.Num);
	* ...
	* auto Functor = [&DstArray, &SrcArray](const FIntRange& Range)->void
	*                {
	*                   for (int i = Range.start(), I = Range.end(); i < I; ++i )
	*                   {
	*                        DstArray[i] = SomeOpperation( SrcArray[i] );
	*                   }
	*                };
	*
	* Parallel_For(FIntRange(0, SrcArray.Num), Functor);
	* -------------------------------------------------------------------------------------------

	* @param Range      RangeType(start, end) describes the range of the for loop
	* @param Functor    Functor with Functor(const RangeType& range) signature performs sub-section of the for loop.
	* @param bParallel  Bool to control parallel vs single threaded (default true). 
	*
	*/
	template <typename RangeType, typename FunctorType>
	void Parallel_For(const RangeType& Range, const FunctorType& Functor, const bool bParallel = true)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::Parallel_For)
		if (bParallel) // run in parallel
		{
			// Functor can be passed by reference since we wait until completion
			tbb::parallel_for(Range,
				[&Functor](const RangeType& Range)
				{
					// #TODO Investigate why Insights stops working when used
					//TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::Parallel_For)
					Functor(Range);
				});
		}
		else // single threaded
		{
			Functor(Range);
		}
	}

	/**
	* Range-based reduction that supports internal range splitting. The work may be split into multiple tasks.
	* Requires a functor that works on a subset of data, and a reduction functor that merges the results of multiple subsets.
	*
	* -------------------------------------------------------------------------------------------
	* Example:  To sum all the numbers in an array of floats
	*
	* TArray<float>  MyArray;
	* ...
	* auto Functor = [&MyArray](const FIntRange& Range, float CurrentSum)->float
	*                {
	*                   for (int i = Range.start(), I = Range.end(); i < I; ++i )
	*                   {
	*                        CurrentSum += MyArray[i];
	*                   }
	*                 return CurrentSum;
	*                };
	*
	* auto ReduceFunctor = [](float A, float B)->float { return A + B; }
	*
	* Sum = Parallel_Reduce(FIntRange(0, MyArray.Num), 0, Functor, ReduceFunctor);
	* -------------------------------------------------------------------------------------------
	*
	*
	* @param Range          RangeType(start, end) describes the range of the work
	* @param Functor        Functor(const RangeType& range, ValueType InitialValue) signature performs sub-section of the work, returns ValueType
	* @param ReduceFunctor  ReduceFunctor(ValueType& A, ValueType& B) signature , returns value type.
	*
	* @param bParallel      Bool to control parallel vs single threaded (default true).
	*
	* @return Result of parallel reduce - of type ValueType.
	*/
	template <typename RangeType, typename ValueType, typename FunctorType, typename ReduceFunctorType>
	ValueType Parallel_Reduce(const RangeType& Range, const ValueType& IdentityValue, const FunctorType& Functor, const ReduceFunctorType& ReduceFunctor, const bool bParallel = true)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::Parallel_Reduce)
		if (bParallel)
		{
			// Functor can be passed by reference since we wait until completion
			return tbb::parallel_reduce(Range, IdentityValue,
				[&Functor](const RangeType& Range, ValueType InitialValue)
				{
					// #TODO Investigate why Insights stops working when used
					//TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::Parallel_Reduce)
					return Functor(Range, InitialValue);
				}, ReduceFunctor);
		}
		else
		{
			return Functor(Range, IdentityValue);
		}
	}

	/**
	* Parallel Task Group - enqueues tasks to be run in parallel.
	*	
	* This class can be constructed with FTrasGroup(false) to force single threaded behavior.
	*
	* NB: The syntax suggest launching (run) individual threads and joining (wait), but in fact this only enqueues task-based functors
	*     in a task manager, and there is no guarantee the tasks will actually be executed on separate threads, or even in parallel.
	*     

	*/
	class FTaskGroup
	{
	public:

		/**
		* A task group constructed with the default constructor will enqueue tasks in the parallel task manager
		*/
		FTaskGroup() {};

		/**
		* Constructor determines if the task group will actually enqueue tasks in the parallel task manager, or
		* simply run them consecutively.
		* 
		* @param Parallel  - should be set to false for debugging to force single threaded behavior. 
		*/
		FTaskGroup(bool Parallel)
			:bParallel(Parallel)
		{};

		/**
		* Enqueue a functor in the task manager. 
		*
		* NB: The functor will directly run if this task group was constructed with Parallel=false;
		*
		* @param Functor  - task to run.
		*/
		template <typename FunctorType>
		void Run(const FunctorType& Functor);
		
		/**
		* Run the functor on the current thread and wait for any other tasks enqueued by this task group to finish.
		*
		* NB: The functor will directly run if this task group was constructed with Parallel=false;
		*
		* @param Functor  - task to run.
		*/
		template <typename FunctorType>
		void RunAndWait(const FunctorType& Functor);
	
		/**
		* Wait for all tasks enqueued by this task group to finish.
		*
		* NB: No op if this task group was constructed with Parallel=false;
		*/
		void Wait()
		{
			if (bParallel)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FTaskGroup::Wait)
				TBBTaskGroup.wait();
			}
		}

	private:
		bool bParallel = true;
		tbb::task_group  TBBTaskGroup;
	};

	template <typename FunctorType>
	void FTaskGroup::Run(const FunctorType& Functor)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTaskGroup::Run)
		if (bParallel)
		{
			// Functor must be passed by copy here since the function returns before completion
			TBBTaskGroup.run(
				[Functor]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTaskGroup::Run)
					Functor();
				}
			);
		}
		else
		{
			Functor();
		}
	}

	template <typename FunctorType>
	void FTaskGroup::RunAndWait(const FunctorType& Functor)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTaskGroup::RunAndWait)
		if (bParallel)
		{
			// Functor can be passed by reference since we wait until completion
			TBBTaskGroup.run_and_wait(
				[&Functor]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTaskGroup::RunAndWait)
					Functor();
				});
		}
		else
		{
			Functor();
		}
	}
}
