// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/UnrealMathUtility.h"

THIRD_PARTY_INCLUDES_START
#include <array>
#include <limits>
THIRD_PARTY_INCLUDES_END

namespace ProxyLOD
{
	/**
	* A double precision class that can be used as a two-dimensional vector.
	* This is used primarily in the rasterization of UV data into a grid that
	* can be used as a texture atlas correspondence table.
	*/
	typedef std::array<double, 2> DVec2d;

	/**
	* Lightweight structure that defines a double precision two-dimensional triangle using 3 DVec2ds.
	*
	*/
	typedef DVec2d DTriangle2d[3];

	/**
	* Vetor-like fucnctions.
	*/
	static double Dot(const DVec2d& V0, const DVec2d& V1);
	static double LengthSqr(const DVec2d& V);
	static double Length(const DVec2d& V);
	static void   Normalize(DVec2d& V);
	static DVec2d Scale(const double ScaleValue, const DVec2d& InV);
	static DVec2d Scale(const DVec2d& InV, const double ScaleValue);
	static DVec2d Subtract(const DVec2d& V0, const DVec2d& V1);
	static DVec2d Add(const DVec2d& V0, const DVec2d& V1);



	/**
	* Compute the axis aligned bounding box of a two-dimensional triangle.  Returned in the form of min and max corners.
	
	*  @param  Triangle      The input 2d triangle.
	*  @param  OutMin        Resulting Min corner
	*  @param  OutMax        Resulting Max cornner 
	*/
	static void ComputeBBox(const DTriangle2d& Triangle, DVec2d& OutBBMin, DVec2d& OutBBMax);
	

	/**
	* Compute the square distance from the input Position to the line segment defined by the two vertices (A and B)
	*
	*  @param  VertA     The start of the line segment.
	*  @param  VertB     The end of the line segment.
	*  @param  Position  Location of point from which to measure the square distance.
	*
	*  @return The shortest distance between the location of Position and the line segment.
	*/
	static double SqrDistanceToSegment(const DVec2d& VertA, const DVec2d& VertB, const DVec2d& Position);
	
	/**
	* Compute the square distance from the input Position to the line segment defined by the two vertices (A and B)
	* And also return the point on the line segment that is closest to the input position
	*
	*  @param  VertA     The start of the line segment.
	*  @param  VertB     The end of the line segment.
	*  @param  Position  Location of point from which to measure the square distance.
	*
	*  @param  OutProjectedPoint The point on the line segment closest to the input location Position.
	*  @param  OutDistSqr        The square distance between the line segment the and location of Position.
	*/
	// Compute the square distance to a line segment and the closest point on the segment
	static void SqrDistanceToSegment(const DVec2d& VertA, const DVec2d& VertB, const DVec2d& Position, DVec2d& OutProjectedPoint, double& OutDstSqr);


	/*
	* Compute the scaled, signed distance from the line  defined by the segment AB to the point P.  Will be positive if P is
	* to the left of the line, and negative if to the right.
	* This is the implicit function for a line between A and B, evaluated at Point.  The units of the result are length
	* of the original line segment ( hence the "scaled" aspect of the result).
	* 
	* @param   A, 2-d point that defines the start of the segment.
	* @param   B, 2-d point that defines the end of the segment.
	* @param   Point, location from which to measure the shortest distance.
	*
	* @return  Scaled, signed distance.
	*/
	static double ScaledDistanceToLine(const DVec2d& A, const DVec2d& B, const DVec2d& Point);

}

// ---- Function implementations ----


static inline double ProxyLOD::Dot(const ProxyLOD::DVec2d& V0, const ProxyLOD::DVec2d& V1)
{
	double Result = V0[0] * V1[0] + V0[1] * V1[1];
	return Result;
}

static inline double ProxyLOD::LengthSqr(const DVec2d& V)
{
	return Dot(V, V);
}
static inline double ProxyLOD::Length(const DVec2d& V)
{
	double LengthSqrValue = LengthSqr(V);
	return FMath::Sqrt(LengthSqrValue);
}
static inline void ProxyLOD::Normalize(DVec2d& V)
{
	double LengthValue = Length(V);
	V[0] /= LengthValue;
	V[1] /= LengthValue;
}

static inline ProxyLOD::DVec2d ProxyLOD::Scale(const double ScaleValue, const ProxyLOD::DVec2d& InV)
{
	DVec2d V = InV;
	V[0] *= ScaleValue;
	V[1] *= ScaleValue;
	return V;
}
static inline ProxyLOD::DVec2d ProxyLOD::Scale(const ProxyLOD::DVec2d& InV, const double ScaleValue)
{
	return Scale(ScaleValue, InV);
}

static inline ProxyLOD::DVec2d ProxyLOD::Subtract(const ProxyLOD::DVec2d& V0, const ProxyLOD::DVec2d& V1)
{
	DVec2d Result{ { V0[0] - V1[0], V0[1] - V1[1] } };
	return Result;
}

static inline ProxyLOD::DVec2d ProxyLOD::Add(const ProxyLOD::DVec2d& V0, const ProxyLOD::DVec2d& V1)
{
	DVec2d Result{ { V0[0] + V1[0], V0[1] + V1[1] } };
	return Result;
}

static inline void ProxyLOD::ComputeBBox(const ProxyLOD::DTriangle2d& Triangle, ProxyLOD::DVec2d& OutBBMin, ProxyLOD::DVec2d& OutBBMax)
{
	OutBBMin = { std::numeric_limits<double>::max(), std::numeric_limits<double>::max() };
	OutBBMax = { std::numeric_limits<double>::lowest(), std::numeric_limits<double>::lowest() };

	for (uint32 V = 0; V < 3; ++V)
	{
		const DVec2d& Vertex = Triangle[V];

		OutBBMin[0] = FMath::Min(OutBBMin[0], Vertex[0]);
		OutBBMin[1] = FMath::Min(OutBBMin[1], Vertex[1]);

		OutBBMax[0] = FMath::Max(OutBBMax[0], Vertex[0]);
		OutBBMax[1] = FMath::Max(OutBBMax[1], Vertex[1]);
	}

}

static inline double ProxyLOD::SqrDistanceToSegment(const ProxyLOD::DVec2d& VertA, const ProxyLOD::DVec2d& VertB, const ProxyLOD::DVec2d& Position)
{
	double OutDstSqr = std::numeric_limits<double>::max();

	// Compute the distance to the end points
	{

		DVec2d TmpVector = Subtract(Position, VertA);
		double TmpOutDstSqr = LengthSqr(TmpVector);
		OutDstSqr = FMath::Min(TmpOutDstSqr, OutDstSqr);

		TmpVector = Subtract(Position, VertB);
		TmpOutDstSqr = LengthSqr(TmpVector);
		OutDstSqr = FMath::Min(TmpOutDstSqr, OutDstSqr);
	}

	// Determine if the Point is in the region perpendicular to the line segment,
	// if so, compute the distance.
	{
		// Construct the edge vector B - A, measure it and record the length sqr
		DVec2d Edge = Subtract(VertB, VertA);
		const double EdgeLength = Length(Edge);
		Normalize(Edge);


		// Vector  P - A
		DVec2d Ray = Subtract(Position, VertA);

		double RayDotEdge = Dot(Ray, Edge);

		// Test if the point in a region perpendicular to the line segment?
		if (RayDotEdge > 0. && RayDotEdge < EdgeLength)
		{

			const DVec2d TangentToEdge = Scale(Edge, RayDotEdge);
			const DVec2d NormalToEdge = Subtract(Ray, TangentToEdge);
			double DstToEdgeSqr = LengthSqr(NormalToEdge);
			OutDstSqr = FMath::Min(OutDstSqr, DstToEdgeSqr);
		}
	}

	return OutDstSqr;
}

static inline void ProxyLOD::SqrDistanceToSegment(const ProxyLOD::DVec2d& VertA, const ProxyLOD::DVec2d& VertB, const ProxyLOD::DVec2d& Position, ProxyLOD::DVec2d& OutProjectedPoint, double& OutDstSqr)
{
	// Compute the distance to the end points
	{

		DVec2d TmpVector = Subtract(Position, VertA);
		double TmpOutDstSqr = LengthSqr(TmpVector);

		if (TmpOutDstSqr < OutDstSqr)
		{
			OutDstSqr = TmpOutDstSqr;
			OutProjectedPoint = VertA;
		}

		TmpVector = Subtract(Position, VertB);
		TmpOutDstSqr = LengthSqr(TmpVector);

		if (TmpOutDstSqr < OutDstSqr)
		{
			OutDstSqr = TmpOutDstSqr;
			OutProjectedPoint = VertB;
		}
	}

	// Determine if the Point is in the region perpendicular to the line segment,
	// if so, compute the distance.
	{
		// Construct the edge vector B - A, measure it and record the length sqr
		DVec2d Edge = Subtract(VertB, VertA);
		const double EdgeLength = Length(Edge);
		Normalize(Edge);


		// Vector  P - A
		DVec2d Ray = Subtract(Position, VertA);

		double RayDotEdge = Dot(Ray, Edge);

		// Test if the point in a region perpendicular to the line segment?
		if (RayDotEdge > 0. && RayDotEdge < EdgeLength)
		{

			const DVec2d TangentToEdge = Scale(Edge, RayDotEdge);
			const DVec2d NormalToEdge = Subtract(Ray, TangentToEdge);
			double DstToEdgeSqr = LengthSqr(NormalToEdge);

			if (DstToEdgeSqr < OutDstSqr)
			{
				OutDstSqr = DstToEdgeSqr;
				OutProjectedPoint = Add(TangentToEdge, VertA);
			}
		}
	}
}

static inline double ProxyLOD::ScaledDistanceToLine(const DVec2d& A, const DVec2d& B, const DVec2d& Point)
{
	return (Point[0] - A[0]) * (A[1] - B[1]) + (B[0] - A[0]) * (Point[1] - A[1]);
}