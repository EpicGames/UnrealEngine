// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"

// Implement vertex types that meet the required templated interface for the simplifier.

/**
* Class that represents the minimal vertex for the simplifier.
* No attributes are associated with the position.
*
* Implements the interface needed by templated simplifier code.
*/
class FPositionOnlyVertex
{
	typedef FPositionOnlyVertex  VertType;
public:
	enum { NumAttributesValue = 0 }; // djh: this should really be 0, but the TQuadricAttrOptimizer will freakout..

	FPositionOnlyVertex() {}

	uint32          MaterialIndex = 0;
	FVector3f			Position;

	FPositionOnlyVertex(const FPositionOnlyVertex& OtherVert) :
		MaterialIndex(OtherVert.MaterialIndex),
		Position(OtherVert.Position)
	{}


	// Attributes are all assumed to be floats.  
	static uint32 NumAttributes() {
		return FPositionOnlyVertex::NumAttributesValue;
	}

	// Methods needed by the simplifier.
	FVector3f&		GetPos() { return Position; }
	const FVector3f&	GetPos() const { return Position; }

	// Required for the simplifier
	float*			GetAttributes() { return nullptr; }
	const float*	GetAttributes() const { return nullptr; }

	// Required for simplifier
	void Correct() {}

	// Note this uses exact float compares..
	bool operator==(const VertType& OtherVert) const
	{
		bool Result = true;
		if (Position != OtherVert.Position)
		{
			Result = false;
		}

		return Result;
	}
	bool operator!=(const VertType& OtherVert)
	{
		return !(*this == OtherVert);
	}

	VertType operator+(const VertType& OtherVert) const
	{
		VertType Result(*this);

		Result.Position += OtherVert.Position;

		return Result;
	}

	// NB: A-B != -(B-A) because of the material index.
	VertType operator-(const VertType& OtherVert) const
	{
		VertType Result(*this);

		Result.Position -= OtherVert.Position;

		return Result;
	}

	// NB: The normal isn't unit length after this..
	VertType operator*(const float Scalar) const
	{
		VertType Result(*this);
		Result.Position *= Scalar;

		return Result;
	}


	VertType operator/(const float Scalar) const
	{
		float ia = 1.0f / Scalar;
		return (*this) * ia;
	}
};

/**
* Simplifier vertex type that stores position and normal.
*
* Implements the interface needed by templated simplifier code.
*/
class FPositionNormalVertex
{
	typedef FPositionNormalVertex  VertType;
public:
	FPositionNormalVertex() {}

	enum { NumAttributesValue = 3 };

	uint32          MaterialIndex = 0;
	FVector3f			Position;
	// ---- Attributes --------
	// currently just the components of the normal.
	FVector3f			Normal;


	// Attributes are all assumed to be floats.  
	static uint32 NumAttributes()
	{

		return (sizeof(VertType) - sizeof(uint32) - sizeof(FVector3f)) / sizeof(float);
	}

	// Methods needed by the simplifier.
	uint32			GetMaterialIndex() const { return MaterialIndex; }
	FVector3f&		GetPos() { return Position; }
	const FVector3f&	GetPos() const { return Position; }
	float*			GetAttributes() { return (float*)&Normal; }
	const float*	GetAttributes() const { return (const float*)&Normal; }


	void Correct() { Normal.Normalize(); }

	// Note this uses exact float compares..
	bool operator==(const VertType& OtherVert) const
	{
		bool Result = true;
		if (MaterialIndex != OtherVert.MaterialIndex ||
			Position != OtherVert.Position ||
			Normal != OtherVert.Normal)
		{
			Result = false;
		}

		return Result;
	}
	bool operator!=(const VertType& OtherVert)
	{
		return !(*this == OtherVert);
	}

	// NB: Addition isn't commutative since the MaterialIndex..
	// maybe Result.MaterialIndex = min(this->MaterialIndex, Other.MaterialIndex) 
	// would be better.
	VertType operator+(const VertType& OtherVert) const
	{
		VertType Result(*this);

		Result.Position += OtherVert.Position;
		Result.Normal += OtherVert.Normal;

		return Result;
	}

	// NB: A-B != -(B-A) because of the material index.
	VertType operator-(const VertType& OtherVert) const
	{
		VertType Result(*this);

		Result.Position -= OtherVert.Position;
		Result.Normal -= OtherVert.Normal;

		return Result;
	}

	// NB: The normal isn't unit length after this..
	VertType operator*(const float Scalar) const
	{
		VertType Result(*this);
		Result.Position *= Scalar;
		Result.Normal *= Scalar;

		return Result;
	}


	VertType operator/(const float Scalar) const
	{
		float ia = 1.0f / Scalar;
		return (*this) * ia;
	}
};
