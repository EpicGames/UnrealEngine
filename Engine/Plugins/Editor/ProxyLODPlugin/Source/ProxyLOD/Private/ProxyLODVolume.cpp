// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODVolume.h"

#include "ProxyLODMeshAttrTransfer.h"
#include "ProxyLODMeshConvertUtils.h"
#include "ProxyLODMeshSDFConversions.h"
#include "ProxyLODMeshTypes.h"
#include "ProxyLODMeshUtilities.h"

THIRD_PARTY_INCLUDES_START
#pragma warning(push)
#pragma warning(disable: 4146)
#include <openvdb/openvdb.h>
#include <openvdb/tools/Interpolation.h> // for Spatial Query
#include <openvdb/tools/MeshToVolume.h> // for MeshToVolume
#include <openvdb/tools/VolumeToMesh.h> // for VolumeToMesh
#include <openvdb/tools/Composite.h> // for CSG operations
#pragma warning(pop)
THIRD_PARTY_INCLUDES_END

#include "MeshDescription.h"

typedef openvdb::math::Transform OpenVDBTransform;

class FProxyLODVolumeImpl : public IProxyLODVolume
{
public:
	FProxyLODVolumeImpl()
		: VoxelSize(0.0)
	{
	}

	~FProxyLODVolumeImpl()
	{
		SDFVolume.reset();
		SrcPolyIndexGrid.reset();
		Sampler.reset();
	}

	bool Initialize(const TArray<FMeshMergeData>& Geometry, float Accuracy)
	{
		FMeshDescriptionArrayAdapter SrcGeometryAdapter(Geometry);
		return Initialize(SrcGeometryAdapter, Accuracy);
	}

	bool Initialize(const TArray<FInstancedMeshMergeData>& Geometry, float Accuracy)
	{
		FMeshDescriptionArrayAdapter SrcGeometryAdapter(Geometry);
		return Initialize(SrcGeometryAdapter, Accuracy);
	}

	virtual double GetVoxelSize() const override
	{
		return VoxelSize;
	}

	virtual FVector3i GetBBoxSize() const override
	{
		if (SDFVolume == nullptr)
		{
			return FVector3i(0,0,0);
		}

		openvdb::math::Coord VolumeBBoxSize = SDFVolume->evalActiveVoxelDim();

		return FVector3i(VolumeBBoxSize.x(), VolumeBBoxSize.y(), VolumeBBoxSize.z());
	}

	virtual void CloseGaps(const double GapRadius, const int32 MaxDilations) override
	{
		ProxyLOD::CloseGaps(SDFVolume, GapRadius, MaxDilations);
	}

	virtual float QueryDistance(const FVector& Point) const override
	{
		return Sampler->wsSample(openvdb::Vec3R(Point.X, Point.Y, Point.Z));
	}

	virtual void ConvertToRawMesh(FMeshDescription& OutRawMesh) const override
	{
		// Mesh types that will be shared by various stages.
		FAOSMesh AOSMeshedVolume;
		ProxyLOD::SDFVolumeToMesh(SDFVolume, 0.0, 0.0, AOSMeshedVolume);
		ProxyLOD::ConvertMesh(AOSMeshedVolume, OutRawMesh);
	}

	void ExpandNarrowBand(float ExteriorWidth, float InteriorWidth) override
	{
		using namespace openvdb::tools;

		FMeshDescription RawMesh;
		FStaticMeshAttributes(RawMesh).Register();
		ConvertToRawMesh(RawMesh);
		FMeshDescriptionAdapter MeshAdapter(RawMesh, SDFVolume->transform());

		openvdb::FloatGrid::Ptr NewSDFVolume;
		openvdb::Int32Grid::Ptr NewSrcPolyIndexGrid;

		try
		{
			NewSrcPolyIndexGrid = openvdb::Int32Grid::create();
			NewSDFVolume = openvdb::tools::meshToVolume<openvdb::FloatGrid>(MeshAdapter, MeshAdapter.GetTransform(), ExteriorWidth / VoxelSize, InteriorWidth / VoxelSize, 0, NewSrcPolyIndexGrid.get());

			SDFVolume = NewSDFVolume;
			SrcPolyIndexGrid = NewSrcPolyIndexGrid;

			// reduce memory footprint, increase the spareness.
			openvdb::tools::pruneLevelSet(SDFVolume->tree(), float(ExteriorWidth / VoxelSize), float(-InteriorWidth / VoxelSize));
		}
		catch (std::bad_alloc&)
		{
			NewSDFVolume.reset();
			NewSrcPolyIndexGrid.reset();
			return;
		}

		Sampler.reset(new openvdb::tools::GridSampler<openvdb::FloatGrid, openvdb::tools::PointSampler>(*SDFVolume));
	}

private:
	bool Initialize(FMeshDescriptionArrayAdapter& InSrcGeometryAdapter, float Accuracy)
	{
		OpenVDBTransform::Ptr XForm = OpenVDBTransform::createLinearTransform(Accuracy);
		InSrcGeometryAdapter.SetTransform(XForm);

		VoxelSize = InSrcGeometryAdapter.GetTransform().voxelSize()[0];

		SrcPolyIndexGrid = openvdb::Int32Grid::create();

		if (!ProxyLOD::MeshToSDFVolume(InSrcGeometryAdapter, InSrcGeometryAdapter.GetTransform(), SDFVolume, SrcPolyIndexGrid.get()))
		{
			SrcPolyIndexGrid.reset();
			return false;
		}

		Sampler.reset(new openvdb::tools::GridSampler<openvdb::FloatGrid, openvdb::tools::PointSampler>(*SDFVolume));

		return true;
	}

	openvdb::FloatGrid::Ptr SDFVolume;
	openvdb::Int32Grid::Ptr SrcPolyIndexGrid;
	openvdb::tools::GridSampler<openvdb::FloatGrid, openvdb::tools::PointSampler>::Ptr Sampler;
	double VoxelSize;
};

int32 IProxyLODVolume::FVector3i::MinIndex() const
{
	return (int32)openvdb::math::MinIndex(openvdb::math::Coord(X, Y, X));
}

TUniquePtr<IProxyLODVolume> IProxyLODVolume::CreateSDFVolumeFromMeshArray(const TArray<FMeshMergeData>& Geometry, float Step)
{
	TUniquePtr<FProxyLODVolumeImpl> Volume = MakeUnique<FProxyLODVolumeImpl>();

	if (Volume == nullptr || !Volume->Initialize(Geometry, Step))
	{
		return nullptr;
	}

	return Volume;
}

TUniquePtr<IProxyLODVolume> IProxyLODVolume::CreateSDFVolumeFromMeshArray(const TArray<FInstancedMeshMergeData>& Geometry, float Step)
{
	TUniquePtr<FProxyLODVolumeImpl> Volume = MakeUnique<FProxyLODVolumeImpl>();

	if (Volume == nullptr || !Volume->Initialize(Geometry, Step))
	{
		return nullptr;
	}

	return Volume;
}

class FPolygonSoup
{
public:
	FPolygonSoup(const TArray<FMeshDescriptionAdapter>& AdapterArray, double VoxelSize);

	// Total number of polygons
	size_t polygonCount() const { return NumPolys; }

	// Total number of points (vertex locations)
	size_t pointCount() const { return NumVerts; }

	// Vertex count for polygon n: currently FMeshDescription is just triangles.
	size_t vertexCount(size_t n) const { return 3; }

	// Return position pos in local grid index space for polygon n and vertex v
	void getIndexSpacePoint(size_t FaceNumber, size_t CornerNumber, openvdb::Vec3d& pos) const
	{
		int32 AdapterIdx = PolyIdxToAdapter[FaceNumber];
		int32 Offset = AdapterToOffset[AdapterIdx];

		Adapters[AdapterIdx].getIndexSpacePoint(FaceNumber - Offset, CornerNumber, pos);
	}

	const OpenVDBTransform& Transform() const { return *Xform; }

private:

	FPolygonSoup();

	OpenVDBTransform::Ptr Xform;

	TArray<FMeshDescriptionAdapter> Adapters;
	TArray<size_t> AdapterToOffset;
	TArray<int32> PolyIdxToAdapter;
	size_t NumPolys;
	size_t NumVerts;

};

FPolygonSoup::FPolygonSoup(const TArray<FMeshDescriptionAdapter>& AdapterArray, double VoxelSize)
{

	Xform = OpenVDBTransform::createLinearTransform(VoxelSize);

	int32 NumMeshes = AdapterArray.Num();
	Adapters.Reserve(NumMeshes);

	NumPolys = 0;
	NumVerts = 0;
	for (auto& Adapter : AdapterArray)
	{
		Adapters.Add(Adapter);
		NumPolys += Adapter.polygonCount();
		NumVerts += Adapter.pointCount();
	}

	AdapterToOffset.AddZeroed(NumMeshes + 1);
	for (int32 i = 1; i < NumMeshes + 1; ++i)
	{
		AdapterToOffset[i] = AdapterToOffset[i-1] + AdapterArray[i - 1].polygonCount();
	}

	PolyIdxToAdapter.Reserve(NumPolys);
	for (int32 i = 0; i < NumMeshes; ++i)
	{
		int32 np = AdapterArray[i].polygonCount();
		for (int32 j = 0; j < np; ++j)
		{
			PolyIdxToAdapter.Add(i);
		}
	}
}





class FVoxelBasedCSGImpl : public IVoxelBasedCSG
{
public:
	FVoxelBasedCSGImpl() 
	{
		openvdb::initialize();

		double VoxelSize = 0.1;
		XForm = OpenVDBTransform::createLinearTransform(VoxelSize);
	}

	FVoxelBasedCSGImpl(double VoxelSize)
	{
		openvdb::initialize();

		XForm = OpenVDBTransform::createLinearTransform(VoxelSize);
	}

	virtual ~FVoxelBasedCSGImpl()
	{
		XForm.reset();
	}

	double GetVoxelSize() const override
	{
		return XForm->voxelSize()[0];
	}

	void SetVoxelSize(double VoxelSize) override 
	{
		XForm = OpenVDBTransform::createLinearTransform(VoxelSize);
	}

	// Note this will become very slow if the isosurface is more than 3 voxel widths from 0.
	// the tool that calls this should clamp the isosurface.
	virtual FVector ComputeUnion(const TArray<IVoxelBasedCSG::FPlacedMesh>& PlacedMeshArray, FMeshDescription& ResultMesh, double Adaptivity, double IsoSurface) const override
	{
		FInterrupter NullInterrupter;
		FVector AverageTranslation;
		ComputeUnion(NullInterrupter, PlacedMeshArray, ResultMesh, AverageTranslation, Adaptivity, IsoSurface);
		return AverageTranslation;
	}


	virtual bool  ComputeUnion(IVoxelBasedCSG::FInterrupter& Interrupter,  const TArray<IVoxelBasedCSG::FPlacedMesh>& PlacedMeshArray, FMeshDescription& ResultMesh, FVector& AverageTranslation, double Adaptivity, double IsoSurface) const override
	{

		
		// convert IsoSurface units to voxels

		const double VoxelSize = GetVoxelSize();

		const double IsoSurfaceInVoxels = IsoSurface / VoxelSize;

		const double ExteriorVoxelWidth = FMath::Max( 2., IsoSurfaceInVoxels + 1.);
		const double InteriorVoxelWidth = -FMath::Min(-2., IsoSurfaceInVoxels - 1);

		AverageTranslation = FVector(0.f, 0.f, 0.f);

		const int32 NumMeshes = PlacedMeshArray.Num();
		if (NumMeshes == 0)
		{
			return false;
		}

		// Find the average translation of all the meshes.
		
		for (int32 i = 0; i < NumMeshes; ++i)
		{
			AverageTranslation += PlacedMeshArray[i].Transform.GetTranslation();
		}
		AverageTranslation /= (float)NumMeshes;


		// Target grid to hold the union in SDF form.
		//openvdb::FloatGrid::Ptr SDFUnionVolume = openvdb::FloatGrid::create(ExteriorVoxelWidth /*background value */);
		//SDFUnionVolume->setTransform(XForm);
		
		// Fill the SDFUnionVolume
		FMatrix44f LocalToVoxel = FMatrix44f::Identity;
		LocalToVoxel.M[0][0] = VoxelSize;
		LocalToVoxel.M[1][1] = VoxelSize;
		LocalToVoxel.M[2][2] = VoxelSize;

		TArray<FMeshDescriptionAdapter> Adapters;
		for (int32 i = 0, I = PlacedMeshArray.Num(); i < I; ++i)
		{
			const FPlacedMesh& PlacedMesh  = PlacedMeshArray[i];
			const FMeshDescription* MeshPtr = PlacedMesh.Mesh;
			if (MeshPtr)
			{

				// Get the transform relative to the average
				FTransform MeshTransform = PlacedMesh.Transform;
				MeshTransform.AddToTranslation(-AverageTranslation);
				FMatrix44f TransformMatrix = FMatrix44f(MeshTransform.ToMatrixWithScale().Inverse());		// LWC_TODO: Precision loss
				

 				TransformMatrix = LocalToVoxel * TransformMatrix;
				float* data = &TransformMatrix.M[0][0];
				openvdb::math::Mat4<float> VDBMatFloat(data);

				openvdb::Mat4R VDBMatDouble(VDBMatFloat);
				// NB: rounding errors in the inverse may have resulted in error in this col.
				// openvdb explicitly checks this matrix row to insure the tranform is affine and will throw 
				VDBMatDouble.setCol(3, openvdb::Vec4R(0, 0, 0, 1));

				OpenVDBTransform::Ptr LocalXForm = OpenVDBTransform::createLinearTransform(VDBMatDouble);

				// Create a wrapper with OpenVDB semantics. 
				
				FMeshDescriptionAdapter MeshAdapter(*MeshPtr, *LocalXForm);

				Adapters.Add(MeshAdapter);

			}
		}

		FPolygonSoup PolySoup(Adapters, VoxelSize);
		openvdb::FloatGrid::Ptr SDFUnionVolume = openvdb::tools::meshToVolume<openvdb::FloatGrid>(Interrupter, PolySoup, PolySoup.Transform(), ExteriorVoxelWidth, InteriorVoxelWidth);

		const bool bSuccess = !Interrupter.wasInterrupted();
		if (bSuccess)
		{ 
			// Convert the SDFUnionVolume to a mesh
		
			ConvertSDFToMesh(SDFUnionVolume, Adaptivity, IsoSurface, ResultMesh);
		}

		return bSuccess;
		
	}

	
	virtual FVector ComputeDifference(const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, double Adaptivity, double IsoSurface) const override
	{
		IVoxelBasedCSG::FInterrupter NullInterrupter;
		FVector AverageTranslation;
		ComputeDifference(NullInterrupter, PlacedMeshA, PlacedMeshB, ResultMesh, AverageTranslation, Adaptivity, IsoSurface);
		return AverageTranslation;
	}

	virtual bool ComputeDifference(IVoxelBasedCSG::FInterrupter& Interrupter, const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, FVector& AverageTranslation, double Adaptivity, double IsoSurface) const override
	{

		// make SDFs
		openvdb::FloatGrid::Ptr SDFVolumeA;
		openvdb::FloatGrid::Ptr SDFVolumeB;

		bool bSuccess = GenerateVolumes(Interrupter, IsoSurface, PlacedMeshA, PlacedMeshB, SDFVolumeA, SDFVolumeB, AverageTranslation);

		bSuccess = bSuccess && !Interrupter.wasInterrupted();
		if (bSuccess)
		{ 
			// create the difference - result stored in volume A
			openvdb::tools::csgDifference(*SDFVolumeA, *SDFVolumeB);
		}

		// convert the result
		bSuccess = bSuccess && !Interrupter.wasInterrupted();
		if (bSuccess)
		{ 
			ConvertSDFToMesh(SDFVolumeA, Adaptivity, IsoSurface, ResultMesh);
		}
		return bSuccess;
	}

	
	virtual FVector ComputeIntersection(const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, double Adaptivity, double IsoSurface) const override
	{
		IVoxelBasedCSG::FInterrupter NullInterrupter;
		FVector AverageTranslation;
		ComputeIntersection(NullInterrupter, PlacedMeshA, PlacedMeshB, ResultMesh, AverageTranslation, Adaptivity, IsoSurface);
		return AverageTranslation;
	}

	virtual bool ComputeIntersection(IVoxelBasedCSG::FInterrupter& Interrupter,  const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, FVector& AverageTranslation, double Adaptivity, double IsoSurface) const override
	{

		// make SDFs
		openvdb::FloatGrid::Ptr SDFVolumeA;
		openvdb::FloatGrid::Ptr SDFVolumeB;

		bool bSuccess = GenerateVolumes(Interrupter, IsoSurface, PlacedMeshA, PlacedMeshB, SDFVolumeA, SDFVolumeB, AverageTranslation);
		bSuccess = bSuccess && !Interrupter.wasInterrupted();
		if (bSuccess)
		{ 
			// create the difference - result stored in volume A
			openvdb::tools::csgIntersection(*SDFVolumeA, *SDFVolumeB);
		}

		// convert the result
		bSuccess = bSuccess && !Interrupter.wasInterrupted();
		if (bSuccess)
		{
			// convert the result
			ConvertSDFToMesh(SDFVolumeA, Adaptivity, IsoSurface, ResultMesh);
		}
		
		return bSuccess;
	}

	
	
	virtual FVector ComputeUnion(const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, double Adaptivity, double IsoSurface) const override
	{
		FInterrupter NullInterrupter;
		FVector AverageTranslation;
		ComputeUnion(NullInterrupter, PlacedMeshA, PlacedMeshB, ResultMesh, AverageTranslation, Adaptivity, IsoSurface);
		return AverageTranslation;
	}
	
	virtual bool ComputeUnion(IVoxelBasedCSG::FInterrupter& Interrupter, const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, FVector& AverageTranslation, double Adaptivity, double IsoSurface) const override
	{

		// make SDFs
		openvdb::FloatGrid::Ptr SDFVolumeA;
		openvdb::FloatGrid::Ptr SDFVolumeB;

		bool bSuccess = GenerateVolumes(Interrupter, IsoSurface, PlacedMeshA, PlacedMeshB, SDFVolumeA, SDFVolumeB, AverageTranslation);
		bSuccess = bSuccess && !Interrupter.wasInterrupted();
		if (bSuccess)
		{ 
			// create the difference - result stored in volume A
			openvdb::tools::csgUnion(*SDFVolumeA, *SDFVolumeB);
		}

		// convert the result
		bSuccess = bSuccess && !Interrupter.wasInterrupted();
		if (bSuccess)
		{	
			ConvertSDFToMesh(SDFVolumeA, Adaptivity, IsoSurface, ResultMesh);
		}
		return bSuccess;
	}

	

private:

	void ConvertSDFToMesh(openvdb::FloatGrid::Ptr SDFVolume, double Adaptivity, double IsoSurface, FMeshDescription& ResultMesh) const
	{
		// Convert the SDFUnionVolume to a mesh
		FAOSMesh AOSMeshedVolume;
		ProxyLOD::ExtractIsosurfaceWithNormals(SDFVolume, IsoSurface, Adaptivity, AOSMeshedVolume); // QUestion should IsoSurface be worldspace?
		
																									
		// Evidently this conversion code makes certain assumptions about the FMeshDescription. that were introduced when it was converted from FRawMesh...
		
		ProxyLOD::ConvertMesh(AOSMeshedVolume, ResultMesh);
	}

	bool  GenerateVolumes(IVoxelBasedCSG::FInterrupter& Interrupter, const double IsoSurface, const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, openvdb::FloatGrid::Ptr& VolumeA, openvdb::FloatGrid::Ptr& VolumeB, FVector& AverageTranslation) const	
	{
		const FMeshDescription& MeshA = *PlacedMeshA.Mesh;
		const FMeshDescription& MeshB = *PlacedMeshB.Mesh;

		const double VoxelSize = GetVoxelSize();

		const double IsoSurfaceInVoxels = IsoSurface / VoxelSize;


		const double ExteriorVoxelWidth = FMath::Max(2., IsoSurfaceInVoxels + 1.);
		const double InteriorVoxelWidth = -FMath::Min(-2., IsoSurfaceInVoxels - 1);

		// The average translation of the two meshes.
		AverageTranslation = PlacedMeshA.Transform.GetTranslation() + PlacedMeshB.Transform.GetTranslation();
		AverageTranslation *= 0.5f;

		// Fill the SDFUnionVolume
		FMatrix LocalToVoxel = FMatrix::Identity;
		LocalToVoxel.M[0][0] = VoxelSize;
		LocalToVoxel.M[1][1] = VoxelSize;
		LocalToVoxel.M[2][2] = VoxelSize;

		auto TransformGenerator = [&LocalToVoxel, &AverageTranslation](const FPlacedMesh& PlacedMesh)->openvdb::Mat4R
		{
			FTransform MeshXForm = PlacedMesh.Transform;
			MeshXForm.AddToTranslation(-AverageTranslation);
			FMatrix TransformMatrix = MeshXForm.ToMatrixWithScale().Inverse();

			TransformMatrix = LocalToVoxel * TransformMatrix;
			double* data = &TransformMatrix.M[0][0];
			openvdb::Mat4R VDBMatDouble(data);
			// NB: rounding errors in the inverse may have resulted in error in this col.
			// openvdb explicitly checks this matrix row to insure the transform is affine and will throw 
			VDBMatDouble.setCol(3, openvdb::Vec4R(0, 0, 0, 1));
			return VDBMatDouble;
		};

		openvdb::Mat4R XFormA = TransformGenerator(PlacedMeshA);
		OpenVDBTransform::Ptr VDBXFormA = OpenVDBTransform::createLinearTransform(XFormA);

		openvdb::Mat4R XFormB = TransformGenerator(PlacedMeshB);
		OpenVDBTransform::Ptr VDBXFormB = OpenVDBTransform::createLinearTransform(XFormB);

		// Create adapters that understand the openVDB semantics.
		FMeshDescriptionAdapter AdapterA(MeshA, *VDBXFormA);
		FMeshDescriptionAdapter AdapterB(MeshB, *VDBXFormB);

		// target transform
		OpenVDBTransform::Ptr TargetXForm = OpenVDBTransform::createLinearTransform(VoxelSize);

		// make SDFs
		openvdb::FloatGrid::Ptr SDFVolumeA = openvdb::tools::meshToVolume<openvdb::FloatGrid>(Interrupter, AdapterA, *TargetXForm, ExteriorVoxelWidth, InteriorVoxelWidth);
		openvdb::FloatGrid::Ptr SDFVolumeB = openvdb::tools::meshToVolume<openvdb::FloatGrid>(Interrupter, AdapterB, *TargetXForm, ExteriorVoxelWidth, InteriorVoxelWidth);

		VolumeA = SDFVolumeA;
		VolumeB = SDFVolumeB;

		const bool bSuccess = !Interrupter.wasInterrupted();
		return bSuccess;

	}


	FVector GenerateVolumes(const double IsoSurface, const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, openvdb::FloatGrid::Ptr& VolumeA, openvdb::FloatGrid::Ptr& VolumeB) const
	{
		FVector AverageTranslation;
		IVoxelBasedCSG::FInterrupter NullInterrupter;
		GenerateVolumes(NullInterrupter, IsoSurface, PlacedMeshA, PlacedMeshB, VolumeA, VolumeB, AverageTranslation);
		return AverageTranslation;
	}

	OpenVDBTransform::Ptr XForm;
};

TUniquePtr<IVoxelBasedCSG> IVoxelBasedCSG::CreateCSGTool(float VoxelSize)
{
	TUniquePtr<FVoxelBasedCSGImpl> CSGTool = MakeUnique<FVoxelBasedCSGImpl>();

	if (CSGTool == nullptr)
	{
		return nullptr;
	}
	else
	{
		CSGTool->SetVoxelSize(VoxelSize);
	}

	return CSGTool;
}