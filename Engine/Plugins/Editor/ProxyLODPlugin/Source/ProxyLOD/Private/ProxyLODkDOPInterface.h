// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "kDOP.h"

#include "ProxyLODMeshTypes.h"
#include "ProxyLODThreadedWrappers.h"

struct FMeshDescription;

namespace ProxyLOD
{
	/**
	*  Required class used by the kDOP system to account for scale and location of the geometry being stored in the acceleration structure.
	*  In our case the geometry exists in world space and no real transformation is needed.
	*/
	class FUnitTransformDataProvider
	{
	public:
		typedef uint32                                               kDopIndexType;
		typedef TkDOPTree<const FUnitTransformDataProvider, uint32>  kDopTreeType;
		/** Initialization constructor. */
		FUnitTransformDataProvider(
			const TkDOPTree<const FUnitTransformDataProvider, kDopIndexType>& InkDopTree) :
			kDopTree(InkDopTree)
		{}

		// kDOP data provider interface.

		FORCEINLINE const TkDOPTree<const FUnitTransformDataProvider, kDopIndexType>& GetkDOPTree(void) const
		{
			return kDopTree;
		}

		FORCEINLINE const FMatrix& GetLocalToWorld(void) const
		{
			return FMatrix::Identity;
		}

		FORCEINLINE const FMatrix& GetWorldToLocal(void) const
		{
			return FMatrix::Identity;
		}

		FORCEINLINE FMatrix GetLocalToWorldTransposeAdjoint(void) const
		{
			return FMatrix::Identity;
		}

		FORCEINLINE float GetDeterminant(void) const
		{
			return 1.0f;
		}

	private:

		const kDopTreeType& kDopTree;
	};


	/**
	* Typedefs for our version of the k-dop tree and triangles.
	*/
	typedef FUnitTransformDataProvider::kDopTreeType                                FkDOPTree;
	typedef FkDOPBuildCollisionTriangle<FUnitTransformDataProvider::kDopIndexType>  FkDOPBuildTriangle;


	/**
	* Build the k-dop acceleration structure from various triangular mesh formats.  Only the location of the 
	* triangles is important. 
	*
	* NB: Internally these are partially threaded.
	*
	* @param InMesh     Input triangle mesh
	* @param kDOPTree   The resulting acceleration structure.
	*/
	void BuildkDOPTree( const FMeshDescription& MeshDescription, FkDOPTree& kDOPTree);
	void BuildkDOPTree( const FVertexDataMesh& InMesh, FkDOPTree& kDOPTree );
	void BuildkDOPTree( const FMeshDescriptionArrayAdapter& InMesh, FkDOPTree& kDOPTree);
}
