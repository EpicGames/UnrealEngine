// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyLODkDOPInterface.h"

#include "ProxyLODMeshTypes.h"
#include "ProxyLODThreadedWrappers.h"

#include "StaticMeshAttributes.h"

// Utils for building a kdop tree from different mesh types.

void ProxyLOD::BuildkDOPTree(const FMeshDescriptionArrayAdapter& SrcGeometry, ProxyLOD::FkDOPTree& kDOPTree)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::BuildkDOPTree)

	const auto NumSrcPoly = SrcGeometry.polygonCount();

	TArray<FkDOPBuildTriangle> BuildTriangleArray;

	// pre-allocated
	ResizeArray(BuildTriangleArray, NumSrcPoly);

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumSrcPoly),
		[&BuildTriangleArray, &SrcGeometry](const ProxyLOD::FUIntRange& Range)
	{
		FkDOPBuildTriangle* BuildTriangles = BuildTriangleArray.GetData();
		for (uint32 r = Range.begin(), R = Range.end(); r < R; ++r)
		{
			const auto& Poly = SrcGeometry.GetRawPoly(r);
			BuildTriangles[r] = FkDOPBuildTriangle(r, FVector(Poly.VertexPositions[0]), FVector(Poly.VertexPositions[1]), FVector(Poly.VertexPositions[2]));
		}

	});

	// Add everything to the tree.
	kDOPTree.Build(BuildTriangleArray);
}

void ProxyLOD::BuildkDOPTree(const FMeshDescription& MeshDescription, FkDOPTree& kDOPTree)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::BuildkDOPTree)

	TArrayView<const FVector3f> VertexPositions = MeshDescription.GetVertexPositions().GetRawArray();

	uint32 NumSrcPoly = MeshDescription.Triangles().Num();

	TArray<FkDOPBuildTriangle> BuildTriangleArray;

	// pre-allocated
	ResizeArray(BuildTriangleArray, NumSrcPoly);

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumSrcPoly),
		[&BuildTriangleArray, &MeshDescription, &VertexPositions](const ProxyLOD::FUIntRange& Range)
	{
		FkDOPBuildTriangle* BuildTriangles = BuildTriangleArray.GetData();

		for (uint32 r = Range.begin(), R = Range.end(); r < R; ++r)
		{

			FVector Pos[3] = { (FVector)VertexPositions[MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(3 * r))],
								(FVector)VertexPositions[MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(3 * r + 1))],
								(FVector)VertexPositions[MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(3 * r + 2))] };
			BuildTriangles[r] = FkDOPBuildTriangle(r, Pos[0], Pos[1], Pos[2]);
		}

	});

	// Add everything to the tree.
	kDOPTree.Build(BuildTriangleArray);
}

void ProxyLOD::BuildkDOPTree(const FVertexDataMesh& SrcVertexDataMesh, ProxyLOD::FkDOPTree& kDOPTree)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::BuildkDOPTree)

	const auto NumSrcPoly = SrcVertexDataMesh.Indices.Num() / 3;

	TArray<FkDOPBuildTriangle> BuildTriangleArray;

	// pre-allocated
	ResizeArray(BuildTriangleArray, NumSrcPoly);

	ProxyLOD::Parallel_For(ProxyLOD::FUIntRange(0, NumSrcPoly),
		[&BuildTriangleArray, &SrcVertexDataMesh](const ProxyLOD::FUIntRange& Range)
	{
		FkDOPBuildTriangle* BuildTriangles = BuildTriangleArray.GetData();
		const uint32* Idxs = SrcVertexDataMesh.Indices.GetData();
		const FVector3f* Positions = SrcVertexDataMesh.Points.GetData();

		for (uint32 r = Range.begin(), R = Range.end(); r < R; ++r)
		{
			FVector Pos[3] = { (FVector)Positions[Idxs[3 * r]],  (FVector)Positions[Idxs[3 * r + 1]],  (FVector)Positions[Idxs[3 * r + 2]] };
			BuildTriangles[r] = FkDOPBuildTriangle(r, Pos[0], Pos[1], Pos[2]);
		}

	});

	// Add everything to the tree.
	kDOPTree.Build(BuildTriangleArray);
}