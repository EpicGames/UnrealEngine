// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ProxyLODMeshReduction : ModuleRules
	{
		public ProxyLODMeshReduction(ReadOnlyTargetRules Target) : base(Target)
		{
			bEnableExceptions = true;
			bDisableAutoRTFMInstrumentation = true;

			// For boost:: and TBB:: code
			bUseRTTI = true;

			PublicIncludePaths.AddRange(
				new string[] {
					// ... add public include paths required here ...
				}
				);

			PrivateIncludePaths.AddRange(
				new string[] {
					// ... add other private include paths required here ...
				}
				);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"TraceLog",
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"UnrealEd",
					"RawMesh",
					"MeshDescription",
					"StaticMeshDescription",
					"MeshUtilities",
					"MaterialUtilities",
					"PropertyEditor",
					"SlateCore",
					"Slate",
					"RenderCore",
					"RHI",
					"QuadricMeshReduction"
					// ... add other public dependencies that you statically link with here ...
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorFramework",
					// ... add private dependencies that you statically link with here ...
				}
				);

			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"IntelTBB",
				"UVAtlas",
				"DirectXMesh",
				"OpenVDB"
			);
		}
	}
}
