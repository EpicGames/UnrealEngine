// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API PROXYLODMESHREDUCTION_API

struct FMeshDescription;


class IProxyLODParameterization
{
public:
	static UE_API TUniquePtr<IProxyLODParameterization> CreateTool();
	virtual ~IProxyLODParameterization()= default;
	
	virtual bool ParameterizeMeshDescription(FMeshDescription& MeshDescription, 
		                                     int32 Width, 
		                                     int32 Height,
											 float GutterSpace, 
		                                     float Stretch, 
		                                     int32 ChartNum,
											 bool bUseNormals, 
		                                     bool bRecomputeTangentSpace, 
		                                     bool bPrintDebugMessages) const = 0;

	/**
* Lower-level entry point:
* Method that generates new UVs inconstant with the mesh defined by the VertexBuffer/IndexBuffer
* according to the parameters specified in the FTextureAtlasDesc
* The underlying code uses Isometeric approach (Iso-Charts) in UV generation.
* This assumes that the caller has already generated the mesh adjacency information.
*
*
*
* @param InTextureAtlasDesc   Description of the texel resolution of the desired texture atlas.
* @param VertexBuffer         Positions of vertices
* @param IndexBuffer          Triangle definitions
* @param AdjacencyBuffer      3 entries for each face, each entry is the face index of the face that is adjacent to the given face. -1 means no face.
* @param StatusCallBack       Allows for termination of the UV generation.
* @param UVVertexBuffer       Resulting UVs
* @param UVIndexBuffer        Connectivity for the UVs
* @param VertexRemapArray     Array that remaps the vertices split by the UV generation to the source vertices.
* @param MaxStretch           On Input, The maximum amount of stretch between (0 - none and 1 - any), on Output the actual max stretch used.
* @param MaxChartNumber       On Input Maximum number of charts required for the atlas.  If this is 0, will be parameterized solely on stretch.
*                             Note, not a hard limit - isochart will stop when a valid charting is found that is greater or equal to this number.
*                             On Output: Number of charts actually generated
*
* @return 'true' if the UV generation succeeded,  'false' if it failed.
*/
	virtual bool GenerateUVs(int32                    Width,
			                 int32                    Height,
		                     float                    GutterSpace,
			        		 const TArray<FVector3f>&   VertexBuffer,
		                     const TArray<int32>&     IndexBuffer,
		                     const TArray<int32>&     AdjacencyBuffer,
		                     TFunction<bool(float)>&  Callback,
		                     TArray<FVector2D>&       UVVertexBuffer,
		                     TArray<int32>&           UVIndexBuffer,
		                     TArray<int32>&           VertexRemapArray,
		                     float&                   MaxStretch,
		                     int32&                   NumCharts) const = 0;
};

#undef UE_API
