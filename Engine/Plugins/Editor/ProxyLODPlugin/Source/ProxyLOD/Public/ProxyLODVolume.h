// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Vector.h"

#define UE_API PROXYLODMESHREDUCTION_API

struct FMeshMergeData;
struct FInstancedMeshMergeData;
struct FMeshDescription;

class IProxyLODVolume
{
public:
	/** Helper class to extract dimensions in voxel size units of OpenVDB volume */
	struct FVector3i
	{
		int32 X;
		int32 Y;
		int32 Z;

		FVector3i(int32 InX, int32 InY, int32 InZ)
			: X(InX), Y(InY), Z(InZ)
		{
		}

		FORCEINLINE int32 operator[](int32 Index) const
		{
			return Index == 0 ? X : (Index == 1 ? Y : (Index == 2 ? Z : 0));
		}

		UE_API int32 MinIndex() const;
	};

	/** Create OpenVDB volume from input geometry */
	static UE_API TUniquePtr<IProxyLODVolume> CreateSDFVolumeFromMeshArray(const TArray<FMeshMergeData>& Geometry, float Step);
	static UE_API TUniquePtr<IProxyLODVolume> CreateSDFVolumeFromMeshArray(const TArray<FInstancedMeshMergeData>& Geometry, float Step);

	virtual ~IProxyLODVolume() = default;

	/** Get size of voxel cell */
	virtual double GetVoxelSize() const = 0;

	/** Get dimensions of bounding box of OpenVDB volume in multiple of size of voxel cell */
	virtual FVector3i GetBBoxSize() const = 0;

	/** Close any gap in OpenVDB volume which radius is less than given one in given maximum iteration */
	virtual void CloseGaps(const double GapRadius, const int32 MaxDilations) = 0;

	/** Extract iso distance 0 from OpenVDB volume as a RawMesh */
	virtual void ConvertToRawMesh(FMeshDescription& OutRawMesh) const = 0;

	/** Expand exterior and interior narrow band of OpenVDB volume by given amount */
	virtual void ExpandNarrowBand(float ExteriorWidth, float InteriorWidth) = 0;

	/**
	 * Returns distance between given point and iso distance 0 of OpenVDB volume.
	 * Note: Returned value is clamped between -'dimension of interior narrow band' and +'dimension of exterior narrow band'
	*/
	virtual float QueryDistance(const FVector& Point) const = 0;
};




class IVoxelBasedCSG
{
public :
	static UE_API TUniquePtr<IVoxelBasedCSG> CreateCSGTool(float VoxelSize);

	virtual ~IVoxelBasedCSG() = default;

	class FPlacedMesh
	{
	public:
		FPlacedMesh()
			: Mesh(nullptr)
		{}

		explicit FPlacedMesh(const FMeshDescription* MeshIn, const FTransform& TransformIn = FTransform::Identity)
			: Mesh(MeshIn), Transform(TransformIn)
		{}

		FPlacedMesh(const FPlacedMesh& other)
			: Mesh(other.Mesh)
			, Transform(other.Transform)
		{}

		FPlacedMesh& operator=(const FPlacedMesh& other)
		{
			Mesh = other.Mesh;
			Transform = other.Transform;
			return *this;
		}

		const FMeshDescription* Mesh;
		FTransform       Transform;
	};

	// the interrupter interface. 
	struct FInterrupter
	{
	
		FInterrupter() = default;
		virtual ~FInterrupter() = default;
		// templated openvdb code requires these functions.
		virtual void start(const char* name = nullptr) { (void)name; }
		virtual void end() {};
		virtual bool wasInterrupted(int percent = -1) { (void)percent; return false; }
	};

	virtual double GetVoxelSize() const = 0;
	virtual void SetVoxelSize(double VolexSize) = 0;

	// Will destroy UVs and other attributes  Returns the average location of the input meshes 
	virtual FVector ComputeUnion(const TArray<IVoxelBasedCSG::FPlacedMesh>& MeshArray, FMeshDescription& ResultMesh, double Adaptivity = 0.1, double IsoSurfcae = 0.) const = 0;
	
	virtual FVector ComputeDifference(const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, double Adaptivity, double IsoSurface) const = 0;
	virtual FVector ComputeIntersection(const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, double Adaptivity, double IsoSurface) const = 0;
	virtual FVector ComputeUnion(const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, double Adaptivity, double IsoSurface) const = 0;
	
	
	// interruptible versions. return true on success.  
	virtual bool ComputeUnion(IVoxelBasedCSG::FInterrupter& Interrupter, const TArray<IVoxelBasedCSG::FPlacedMesh>& MeshArray, FMeshDescription& ResultMesh, FVector& AverageTranslation, double Adaptivity, double IsoSurfcae) const = 0;
	
	virtual bool ComputeDifference(IVoxelBasedCSG::FInterrupter& Interrupter, const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, FVector& AverageTranslation, double Adaptivity, double IsoSurface) const = 0;
	virtual bool ComputeIntersection(IVoxelBasedCSG::FInterrupter& Interrupter, const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, FVector& AverageTranslation, double Adaptivity, double IsoSurface) const = 0;
	virtual bool ComputeUnion(IVoxelBasedCSG::FInterrupter& Interrupter, const FPlacedMesh& PlacedMeshA, const FPlacedMesh& PlacedMeshB, FMeshDescription& ResultMesh, FVector& AverageTranslation, double Adaptivity, double IsoSurface) const = 0;
};

#undef UE_API
